{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Sequencer Scripting",
	"Description": "Python and editor utility scripting extensions for sequencer and movie scenes",
	"Category": "Scripting",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": ["LiveLinkHub"],
	"Plugins": [
		{
			"Name": "PythonScriptPlugin",
			"Enabled": true
		},
	],
	"Modules": [
		{
			"Name": "SequencerScripting",
			"Type": "Runtime",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "Default"
		},
		{
			"Name": "SequencerScriptingEditor",
			"Type": "Editor",
			"ProgramAllowList": [ "LiveLinkHub" ],
			"LoadingPhase": "Default"
		}
	]
}