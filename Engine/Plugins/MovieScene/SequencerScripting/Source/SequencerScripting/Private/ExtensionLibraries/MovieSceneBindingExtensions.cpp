// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneBindingExtensions.h"
#include "ExtensionLibraries/MovieSceneSequenceExtensions.h"
#include "MovieSceneBindingProxy.h"
#include "MovieScenePossessable.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "MovieSceneSpawnable.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "MovieSceneBindingReferences.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/MovieSceneEvaluationState.h"
#include "MovieSceneCommonHelpers.h"
#include "Engine/World.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBindingExtensions)

bool UMovieSceneBindingExtensions::IsValid(const FMovieSceneBindingProxy& InBinding)
{
	UMovieScene* MovieScene = InBinding.Sequence ? InBinding.Sequence->GetMovieScene() : nullptr;
	if (MovieScene && InBinding.BindingID.IsValid())
	{
		return MovieScene->FindBinding(InBinding.BindingID) != nullptr;
	}

	return false;
}

bool UMovieSceneBindingExtensions::EqualEqual_MovieSceneBindingProxy(FMovieSceneBindingProxy A, FMovieSceneBindingProxy B)
{
	return A == B;
}

bool UMovieSceneBindingExtensions::NotEqual_MovieSceneBindingProxy(FMovieSceneBindingProxy A, FMovieSceneBindingProxy B)
{
	return A != B;
}

FGuid UMovieSceneBindingExtensions::GetId(const FMovieSceneBindingProxy& InBinding)
{
	return InBinding.BindingID;
}

FText UMovieSceneBindingExtensions::GetDisplayName(const FMovieSceneBindingProxy& InBinding)
{
	UMovieScene* MovieScene = InBinding.Sequence ? InBinding.Sequence->GetMovieScene() : nullptr;
	if (MovieScene && InBinding.BindingID.IsValid())
	{
		return MovieScene->GetObjectDisplayName(InBinding.BindingID);
	}

	return FText();
}

void UMovieSceneBindingExtensions::SetDisplayName(const FMovieSceneBindingProxy& InBinding, const FText& InDisplayName)
{
	UMovieScene* MovieScene = InBinding.Sequence ? InBinding.Sequence->GetMovieScene() : nullptr;
	if (MovieScene && InBinding.BindingID.IsValid())
	{
#if WITH_EDITORONLY_DATA
		MovieScene->Modify();
		MovieScene->SetObjectDisplayName(InBinding.BindingID, InDisplayName);
#endif
	}
}

FString UMovieSceneBindingExtensions::GetName(const FMovieSceneBindingProxy& InBinding)
{
	UMovieScene* MovieScene = InBinding.Sequence ? InBinding.Sequence->GetMovieScene() : nullptr;
	if (MovieScene && InBinding.BindingID.IsValid())
	{
		FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(InBinding.BindingID);
		if (Spawnable)
		{
			return Spawnable->GetName();
		}

		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(InBinding.BindingID);
		if (Possessable)
		{
			return Possessable->GetName();
		}
	}

	return FString();
}

void UMovieSceneBindingExtensions::SetName(const FMovieSceneBindingProxy& InBinding, const FString& InName)
{
	UMovieScene* MovieScene = InBinding.Sequence ? InBinding.Sequence->GetMovieScene() : nullptr;
	if (MovieScene && InBinding.BindingID.IsValid())
	{
		MovieScene->Modify();

		FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(InBinding.BindingID);
		if (Spawnable)
		{
			Spawnable->SetName(InName);
		}

		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(InBinding.BindingID);
		if (Possessable)
		{
			Possessable->SetName(InName);
		}
	}
}

int32 UMovieSceneBindingExtensions::GetSortingOrder(const FMovieSceneBindingProxy& InBinding)
{
#if WITH_EDITORONLY_DATA
	UMovieScene* MovieScene = InBinding.GetMovieScene();
	if (MovieScene)
	{
		if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(InBinding.BindingID))
		{
			return Binding->GetSortingOrder();
		}
	}
#endif

	FFrame::KismetExecutionMessage(TEXT("Cannot find requested binding"), ELogVerbosity::Error);
	return 0;
}

void UMovieSceneBindingExtensions::SetSortingOrder(const FMovieSceneBindingProxy& InBinding, int32 SortingOrder)
{
#if WITH_EDITORONLY_DATA
	UMovieScene* MovieScene = InBinding.GetMovieScene();
	if (MovieScene)
	{
		FMovieSceneBinding* Binding = MovieScene->FindBinding(InBinding.BindingID);
		if (Binding)
		{
			Binding->SetSortingOrder(SortingOrder);
			return;
		}
	}
#endif
	FFrame::KismetExecutionMessage(TEXT("Cannot find requested binding"), ELogVerbosity::Error);
}

TArray<UMovieSceneTrack*> UMovieSceneBindingExtensions::GetTracks(const FMovieSceneBindingProxy& InBinding)
{
	UMovieScene* MovieScene = InBinding.GetMovieScene();
	if (MovieScene)
	{
		if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(InBinding.BindingID))
		{
			return Binding->GetTracks();
		}
	}
	return TArray<UMovieSceneTrack*>();
}

void UMovieSceneBindingExtensions::RemoveTrack(const FMovieSceneBindingProxy& InBinding, UMovieSceneTrack* TrackToRemove)
{
	if (!TrackToRemove)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveTrack on a null track"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = InBinding.GetMovieScene();
	if (TrackToRemove && MovieScene)
	{
		MovieScene->RemoveTrack(*TrackToRemove);
	}
}

void UMovieSceneBindingExtensions::Remove(const FMovieSceneBindingProxy& InBinding)
{
	UMovieScene* MovieScene = InBinding.GetMovieScene();
	if (MovieScene)
	{
		if (!MovieScene->RemovePossessable(InBinding.BindingID))
		{
			MovieScene->RemoveSpawnable(InBinding.BindingID);
		}
	}
}

TArray<UMovieSceneTrack*> UMovieSceneBindingExtensions::FindTracksByType(const FMovieSceneBindingProxy& InBinding, TSubclassOf<UMovieSceneTrack> TrackType)
{
	UMovieScene* MovieScene   = InBinding.GetMovieScene();
	UClass*      DesiredClass = TrackType.Get();

	if (MovieScene)
	{
		if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(InBinding.BindingID))
		{
			bool bExactMatch = false;
			return UMovieSceneSequenceExtensions::FilterTracks(Binding->GetTracks(), DesiredClass, bExactMatch);
		}
	}
	return TArray<UMovieSceneTrack*>();
}

TArray<UMovieSceneTrack*> UMovieSceneBindingExtensions::FindTracksByExactType(const FMovieSceneBindingProxy& InBinding, TSubclassOf<UMovieSceneTrack> TrackType)
{
	UMovieScene* MovieScene   = InBinding.GetMovieScene();
	UClass*      DesiredClass = TrackType.Get();

	if (MovieScene)
	{
		if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(InBinding.BindingID))
		{
			bool bExactMatch = true;
			return UMovieSceneSequenceExtensions::FilterTracks(Binding->GetTracks(), DesiredClass, bExactMatch);
		}
	}
	return TArray<UMovieSceneTrack*>();
}

UMovieSceneTrack* UMovieSceneBindingExtensions::AddTrack(const FMovieSceneBindingProxy& InBinding, TSubclassOf<UMovieSceneTrack> TrackType)
{
	UMovieScene* MovieScene   = InBinding.GetMovieScene();
	UClass*      DesiredClass = TrackType.Get();

	if (MovieScene)
	{
		const bool bBindingExists = MovieScene->FindBinding(InBinding.BindingID) != nullptr;
		if (bBindingExists)
		{
			UMovieSceneTrack* NewTrack = NewObject<UMovieSceneTrack>(MovieScene, DesiredClass, NAME_None, RF_Transactional);
			if (NewTrack)
			{
				MovieScene->AddGivenTrack(NewTrack, InBinding.BindingID);
				return NewTrack;
			}
		}
	}
	return nullptr;
}

TArray<FMovieSceneBindingProxy> UMovieSceneBindingExtensions::GetChildPossessables(const FMovieSceneBindingProxy& InBinding)
{
	TArray<FMovieSceneBindingProxy> Result;

	UMovieScene* MovieScene = InBinding.GetMovieScene();
	if (MovieScene)
	{
		FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(InBinding.BindingID);
		if (Spawnable)
		{
			for (const FGuid& ChildGuid : Spawnable->GetChildPossessables())
			{
				Result.Emplace(ChildGuid, InBinding.Sequence);
			}
			return Result;
		}

		const int32 Count = MovieScene->GetPossessableCount();
		for (int32 i = 0; i < Count; ++i)
		{
			FMovieScenePossessable& PossessableChild = MovieScene->GetPossessable(i);
			if (PossessableChild.GetParent() == InBinding.BindingID)
			{
				Result.Emplace(PossessableChild.GetGuid(), InBinding.Sequence);
			}
		}
	}
	return Result;
}

UObject* UMovieSceneBindingExtensions::GetObjectTemplate(const FMovieSceneBindingProxy& InBinding)
{
	UMovieScene* MovieScene = InBinding.GetMovieScene();
	if (MovieScene)
	{
		UMovieSceneSequence* ThisSequence = MovieScene->GetTypedOuter<UMovieSceneSequence>();
		TSharedRef<UE::MovieScene::FSharedPlaybackState> TransientPlaybackState = MovieSceneHelpers::CreateTransientSharedPlaybackState(GWorld, ThisSequence);


		// TODO: Technically this assumes only one spawnable- do we need to upgrade script to handle multiple binding indices?
		return MovieSceneHelpers::GetObjectTemplate(MovieScene->GetTypedOuter<UMovieSceneSequence>(), InBinding.BindingID, TransientPlaybackState);
	}
	return nullptr;
}

UClass* UMovieSceneBindingExtensions::GetPossessedObjectClass(const FMovieSceneBindingProxy& InBinding)
{
	UMovieScene* MovieScene = InBinding.GetMovieScene();
	if (MovieScene)
	{
		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(InBinding.BindingID);
		if (Possessable)
		{
#if WITH_EDITORONLY_DATA
			return const_cast<UClass*>(Possessable->GetPossessedObjectClass());
#endif
		}
	}
	return nullptr;
}

FMovieSceneBindingProxy UMovieSceneBindingExtensions::GetParent(const FMovieSceneBindingProxy& InBinding)
{
	UMovieScene* MovieScene = InBinding.GetMovieScene();
	if (MovieScene)
	{
		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(InBinding.BindingID);
		if (Possessable)
		{
			return FMovieSceneBindingProxy(Possessable->GetParent(), InBinding.Sequence);
		}
	}
	return FMovieSceneBindingProxy();
}


void UMovieSceneBindingExtensions::SetParent(const FMovieSceneBindingProxy& InBinding, const FMovieSceneBindingProxy& InParentBinding)
{
	UMovieScene* MovieScene = InBinding.GetMovieScene();
	if (MovieScene)
	{
		FMovieScenePossessable* Possessable = MovieScene->FindPossessable(InBinding.BindingID);
		if (Possessable)
		{
			MovieScene->Modify();
			Possessable->SetParent(InParentBinding.BindingID, MovieScene);
		}
	}
}

void UMovieSceneBindingExtensions::MoveBindingContents(const FMovieSceneBindingProxy& SourceBindingId, const FMovieSceneBindingProxy& DestinationBindingId)
{
	UMovieScene* MovieScene = SourceBindingId.GetMovieScene();
	if (MovieScene)
	{
		MovieScene->Modify();

		FMovieScenePossessable* SourcePossessable = MovieScene->FindPossessable(SourceBindingId.BindingID);
		FMovieSceneSpawnable* SourceSpawnable = MovieScene->FindSpawnable(SourceBindingId.BindingID);

		FMovieScenePossessable* DestinationPossessable = MovieScene->FindPossessable(DestinationBindingId.BindingID);
		FMovieSceneSpawnable* DestinationSpawnable = MovieScene->FindSpawnable(DestinationBindingId.BindingID);

		if (SourcePossessable && DestinationPossessable)
		{
			MovieScene->MoveBindingContents(SourcePossessable->GetGuid(), DestinationPossessable->GetGuid());
		}
		else if (SourcePossessable && DestinationSpawnable)
		{
			MovieScene->MoveBindingContents(SourcePossessable->GetGuid(), DestinationSpawnable->GetGuid());
		}
		else if (SourceSpawnable && DestinationPossessable)
		{
			MovieScene->MoveBindingContents(SourceSpawnable->GetGuid(), DestinationPossessable->GetGuid());
		}
		else if (SourceSpawnable && DestinationSpawnable)
		{
			MovieScene->MoveBindingContents(SourceSpawnable->GetGuid(), DestinationSpawnable->GetGuid());
		}
	}
}

void UMovieSceneBindingExtensions::SetSpawnableBindingID(const FMovieSceneBindingProxy& InBinding, const FMovieSceneObjectBindingID& SpawnableBindingID)
{
	if (!InBinding.Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetSpawnableBindingID with a binding with a null sequence"), ELogVerbosity::Error);
		return;
	}

	FMovieScenePossessable* Possessable = InBinding.Sequence->GetMovieScene()->FindPossessable(InBinding.BindingID);
	if (!Possessable)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetSpawnableBindingID with a null possessable"), ELogVerbosity::Error);
		return;
	}
	
	Possessable->SetSpawnableObjectBindingID(SpawnableBindingID);
}
