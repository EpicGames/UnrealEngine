// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneBindingTagExtensions.h"
#include "MovieScene.h"
#include "MovieSceneBindingProxy.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequenceID.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBindingTagExtensions)

TArray<FName> UMovieSceneBindingTagExtensions::GetAllBindingTags(UMovieSceneSequence* Sequence)
{
	TArray<FName> Tags;
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get binding tags from a null sequence"), ELogVerbosity::Error);
		return Tags;
	}
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return Tags;
	}
	MovieScene->AllTaggedBindings().GetKeys(Tags);
	return Tags;
}

TArray<FName> UMovieSceneBindingTagExtensions::GetBindingTags(const FMovieSceneBindingProxy& Binding)
{
	TArray<FName> Tags;
	if (!Binding.Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get tags on a binding with a null sequence"), ELogVerbosity::Error);
		return Tags;
	}
	UMovieScene* MovieScene = Binding.Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return Tags;
	}
	const UE::MovieScene::FFixedObjectBindingID Target(
		Binding.BindingID, MovieSceneSequenceID::Root);

	for (const TPair<FName, FMovieSceneObjectBindingIDs>& Pair : MovieScene->AllTaggedBindings())
	{
		for (const FMovieSceneObjectBindingID& ID : Pair.Value.IDs)
		{
			if (ID.ReinterpretAsFixed() == Target)
			{
				Tags.Add(Pair.Key);
				break;
			}
		}
	}
	return Tags;
}

void UMovieSceneBindingTagExtensions::TagBinding(const FMovieSceneBindingProxy& Binding, FName TagName)
{
	if (!Binding.Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot tag a binding with a null sequence"), ELogVerbosity::Error);
		return;
	}
	if (TagName.IsNone())
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot tag a binding with an empty tag name"), ELogVerbosity::Error);
		return;
	}
	UMovieScene* MovieScene = Binding.Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	MovieScene->Modify();

	if (!MovieScene->AllTaggedBindings().Contains(TagName))
	{
		MovieScene->AddNewBindingTag(TagName);
	}

	MovieScene->TagBinding(TagName, UE::MovieScene::FFixedObjectBindingID(
		Binding.BindingID, MovieSceneSequenceID::Root));
}

void UMovieSceneBindingTagExtensions::UntagBinding(const FMovieSceneBindingProxy& Binding, FName TagName)
{
	if (!Binding.Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot untag a binding with a null sequence"), ELogVerbosity::Error);
		return;
	}
	if (TagName.IsNone())
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot untag a binding with an empty tag name"), ELogVerbosity::Error);
		return;
	}
	UMovieScene* MovieScene = Binding.Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	MovieScene->Modify();
	MovieScene->UntagBinding(TagName, UE::MovieScene::FFixedObjectBindingID(
		Binding.BindingID, MovieSceneSequenceID::Root));
}

void UMovieSceneBindingTagExtensions::RemoveBindingTag(UMovieSceneSequence* Sequence, FName TagName)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot remove tag from a null sequence"), ELogVerbosity::Error);
		return;
	}
	if (TagName.IsNone())
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot remove an empty tag name"), ELogVerbosity::Error);
		return;
	}
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene)
	{
		return;
	}

	MovieScene->Modify();
	MovieScene->RemoveTag(TagName);
}
