// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneConditionExtensions.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "Conditions/MovieSceneCondition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneConditionExtensions)

// Section conditions

UMovieSceneCondition* UMovieSceneConditionExtensions::GetSectionCondition(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get condition on a null section"), ELogVerbosity::Error);
		return nullptr;
	}
	return Section->ConditionContainer.Condition;
}

void UMovieSceneConditionExtensions::SetSectionCondition(UMovieSceneSection* Section, TSubclassOf<UMovieSceneCondition> ConditionClass)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot set condition on a null section"), ELogVerbosity::Error);
		return;
	}

	Section->Modify();

	if (!ConditionClass)
	{
		Section->ConditionContainer.Condition = nullptr;
		return;
	}

	Section->ConditionContainer.Condition = NewObject<UMovieSceneCondition>(Section, ConditionClass);
}

void UMovieSceneConditionExtensions::ClearSectionCondition(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot clear condition on a null section"), ELogVerbosity::Error);
		return;
	}

	Section->Modify();
	Section->ConditionContainer.Condition = nullptr;
}

// Track conditions

UMovieSceneCondition* UMovieSceneConditionExtensions::GetTrackCondition(UMovieSceneTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get condition on a null track"), ELogVerbosity::Error);
		return nullptr;
	}
	return Track->ConditionContainer.Condition;
}

void UMovieSceneConditionExtensions::SetTrackCondition(UMovieSceneTrack* Track, TSubclassOf<UMovieSceneCondition> ConditionClass)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot set condition on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->Modify();

	if (!ConditionClass)
	{
		Track->ConditionContainer.Condition = nullptr;
		return;
	}

	Track->ConditionContainer.Condition = NewObject<UMovieSceneCondition>(Track, ConditionClass);
}

void UMovieSceneConditionExtensions::ClearTrackCondition(UMovieSceneTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot clear condition on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->Modify();
	Track->ConditionContainer.Condition = nullptr;
}

// Track row conditions

UMovieSceneCondition* UMovieSceneConditionExtensions::GetTrackRowCondition(UMovieSceneTrack* Track, int32 RowIndex)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get track row condition on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	const FMovieSceneTrackRowMetadata* RowMetadata = Track->FindTrackRowMetadata(RowIndex);
	if (!RowMetadata)
	{
		return nullptr;
	}
	return RowMetadata->ConditionContainer.Condition;
}

void UMovieSceneConditionExtensions::SetTrackRowCondition(UMovieSceneTrack* Track, int32 RowIndex, TSubclassOf<UMovieSceneCondition> ConditionClass)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot set track row condition on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->Modify();

	FMovieSceneTrackRowMetadata& RowMetadata = Track->FindOrAddTrackRowMetadata(RowIndex);

	if (!ConditionClass)
	{
		RowMetadata.ConditionContainer.Condition = nullptr;
		return;
	}

	RowMetadata.ConditionContainer.Condition = NewObject<UMovieSceneCondition>(Track, ConditionClass);
}

void UMovieSceneConditionExtensions::ClearTrackRowCondition(UMovieSceneTrack* Track, int32 RowIndex)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot clear track row condition on a null track"), ELogVerbosity::Error);
		return;
	}

	FMovieSceneTrackRowMetadata* RowMetadata = Track->FindTrackRowMetadata(RowIndex);
	if (RowMetadata)
	{
		Track->Modify();
		RowMetadata->ConditionContainer.Condition = nullptr;
	}
}
