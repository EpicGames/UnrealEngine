// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneEventTrackExtensions.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "Sections/MovieSceneEventRepeaterSection.h"
#include "Sections/MovieSceneEventTriggerSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEventTrackExtensions)

UMovieSceneEventRepeaterSection* UMovieSceneEventTrackExtensions::AddEventRepeaterSection(UMovieSceneEventTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddEventRepeaterSection on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	UMovieSceneEventRepeaterSection* NewSection = NewObject<UMovieSceneEventRepeaterSection>(Track, NAME_None, RF_Transactional);

	if (NewSection)
	{
		Track->Modify();

		Track->AddSection(*NewSection);
	}

	return NewSection;
}

UMovieSceneEventTriggerSection* UMovieSceneEventTrackExtensions::AddEventTriggerSection(UMovieSceneEventTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddEventTriggerSection on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	UMovieSceneEventTriggerSection* NewSection = NewObject<UMovieSceneEventTriggerSection>(Track, NAME_None, RF_Transactional);

	if (NewSection)
	{
		Track->Modify();

		Track->AddSection(*NewSection);
	}

	return NewSection;
}

