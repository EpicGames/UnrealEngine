// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneFolderExtensions.h"
#include "MovieSceneBindingProxy.h"
#include "MovieSceneFolder.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneFolderExtensions)

FName UMovieSceneFolderExtensions::GetFolderName(UMovieSceneFolder* Folder)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetFolderName on a null folder"), ELogVerbosity::Error);
		return FName();
	}

	return Folder->GetFolderName();
}

bool UMovieSceneFolderExtensions::SetFolderName(UMovieSceneFolder* Folder, FName InFolderName)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetFolderName on a null folder"), ELogVerbosity::Error);
		return false;
	}

	Folder->SetFolderName(InFolderName);
	return true;
}

FColor UMovieSceneFolderExtensions::GetFolderColor(UMovieSceneFolder* Folder)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetFolderColor on a null folder"), ELogVerbosity::Error);
		return FColor();
	}

#if WITH_EDITORONLY_DATA
	return Folder->GetFolderColor();
#else
	return FColor();
#endif //WITH_EDITORONLY_DATA
}

bool UMovieSceneFolderExtensions::SetFolderColor(UMovieSceneFolder* Folder, FColor InFolderColor)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetFolderColor on a null folder"), ELogVerbosity::Error);
		return false;
	}

#if WITH_EDITORONLY_DATA
	Folder->Modify();
	Folder->SetFolderColor(InFolderColor);
#endif //WITH_EDITORONLY_DATA
	return true;
}

TArray<UMovieSceneFolder*> UMovieSceneFolderExtensions::GetChildFolders(UMovieSceneFolder* Folder)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetChildFolders on a null folder"), ELogVerbosity::Error);
		return TArray<UMovieSceneFolder*>();
	}

	TArray<UMovieSceneFolder*> Result(Folder->GetChildFolders());
	return Result;
}

bool UMovieSceneFolderExtensions::AddChildFolder(UMovieSceneFolder* TargetFolder, UMovieSceneFolder* FolderToAdd)
{
	if (!TargetFolder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddChildFolder with a null parent folder"), ELogVerbosity::Error);
		return false;
	}

	if (!FolderToAdd)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddChildFolder with a null child folder"), ELogVerbosity::Error);
		return false;
	}

	TargetFolder->AddChildFolder(FolderToAdd);
	return true;
}

bool UMovieSceneFolderExtensions::RemoveChildFolder(UMovieSceneFolder* TargetFolder, UMovieSceneFolder* FolderToRemove)
{
	if (!TargetFolder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveChildFolder with a null parent folder"), ELogVerbosity::Error);
		return false;
	}

	if (!FolderToRemove)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveChildFolder with a null child folder"), ELogVerbosity::Error);
		return false;
	}
	
	TargetFolder->RemoveChildFolder(FolderToRemove);
	return true;
}

TArray<UMovieSceneTrack*> UMovieSceneFolderExtensions::GetChildTracks(UMovieSceneFolder* Folder)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetChildTracks with a null folder"), ELogVerbosity::Error);
		return TArray<UMovieSceneTrack*>();
	}

	return Folder->GetChildTracks();
}

bool UMovieSceneFolderExtensions::AddChildTrack(UMovieSceneFolder* Folder, UMovieSceneTrack* InTrack)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddChildTrack with a null folder"), ELogVerbosity::Error);
		return false;
	}

	if (!InTrack)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddChildTrack with a null track"), ELogVerbosity::Error);
		return false;
	}
	
	Folder->AddChildTrack(InTrack);
	return true;
}

bool UMovieSceneFolderExtensions::RemoveChildTrack(UMovieSceneFolder* Folder, UMovieSceneTrack* InTrack)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveChildTrack with a null folder"), ELogVerbosity::Error);
		return false;
	}

	if (!InTrack)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveChildTrack with a null track"), ELogVerbosity::Error);
		return false;
	}

	Folder->RemoveChildTrack(InTrack);
	return true;
}

TArray<FMovieSceneBindingProxy> UMovieSceneFolderExtensions::GetChildObjectBindings(UMovieSceneFolder* Folder)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetChildObjectBindings with a null folder"), ELogVerbosity::Error);
		return TArray<FMovieSceneBindingProxy>();
	}

	TArray<FMovieSceneBindingProxy> Result;

	// Attempt to get the sequence reference from the folder
	UMovieScene* MovieScene = Cast<UMovieScene>(Folder->GetOuter());
	UMovieSceneSequence* Sequence = Cast<UMovieSceneSequence>(MovieScene->GetOuter());

	for (FGuid ID : Folder->GetChildObjectBindings())
	{
		Result.Add(FMovieSceneBindingProxy(ID, Sequence));
	}

	return Result;
}

bool UMovieSceneFolderExtensions::AddChildObjectBinding(UMovieSceneFolder* Folder, FMovieSceneBindingProxy InObjectBinding)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddChildObjectBinding with a null folder"), ELogVerbosity::Error);
		return false;
	}

	if (InObjectBinding.BindingID.IsValid())
	{
		Folder->AddChildObjectBinding(InObjectBinding.BindingID);
		return true;
	}

	return false;
}

bool UMovieSceneFolderExtensions::RemoveChildObjectBinding(UMovieSceneFolder* Folder, const FMovieSceneBindingProxy InObjectBinding)
{
	if (!Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveChildObjectBinding with a null folder"), ELogVerbosity::Error);
		return false;
	}

	if (InObjectBinding.BindingID.IsValid())
	{
		Folder->RemoveChildObjectBinding(InObjectBinding.BindingID);
		return true;
	}

	return false;
}

