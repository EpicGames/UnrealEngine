// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieScenePrimitiveMaterialTrackExtensions.h"
#include "Tracks/MovieSceneMaterialTrack.h"
#include "Tracks/MovieScenePrimitiveMaterialTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePrimitiveMaterialTrackExtensions)

void UMovieScenePrimitiveMaterialTrackExtensions::SetMaterialIndex(UMovieScenePrimitiveMaterialTrack* Track, const int32 MaterialIndex)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetMaterialIndex on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->Modify();
	Track->SetMaterialInfo(FComponentMaterialInfo{ FName(), MaterialIndex, EComponentMaterialType::IndexedMaterial });
}

int32 UMovieScenePrimitiveMaterialTrackExtensions::GetMaterialIndex(UMovieScenePrimitiveMaterialTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetMaterialIndex on a null track"), ELogVerbosity::Error);
		return -1;
	}

	return Track->GetMaterialInfo().MaterialSlotIndex;
}

void UMovieScenePrimitiveMaterialTrackExtensions::SetMaterialInfo(UMovieScenePrimitiveMaterialTrack* Track, const FComponentMaterialInfo& MaterialInfo)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetMaterialInfo on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->Modify();
	Track->SetMaterialInfo(MaterialInfo);
}

FComponentMaterialInfo UMovieScenePrimitiveMaterialTrackExtensions::GetMaterialInfo(UMovieScenePrimitiveMaterialTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetMaterialInfo on a null track"), ELogVerbosity::Error);
		return FComponentMaterialInfo();
	}

	return Track->GetMaterialInfo();
}








