// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieScenePropertyTrackExtensions.h"
#include "Tracks/MovieSceneByteTrack.h"
#include "Tracks/MovieSceneObjectPropertyTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePropertyTrackExtensions)

void UMovieScenePropertyTrackExtensions::SetPropertyNameAndPath(UMovieScenePropertyTrack* Track, const FName& InPropertyName, const FString& InPropertyPath)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetPropertyNameAndPath on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->Modify();
	Track->SetPropertyNameAndPath(InPropertyName, InPropertyPath);
}

FName UMovieScenePropertyTrackExtensions::GetPropertyName(UMovieScenePropertyTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetPropertyName on a null track"), ELogVerbosity::Error);
		return NAME_None;
	}

	return Track->GetPropertyName();
}

FString UMovieScenePropertyTrackExtensions::GetPropertyPath(UMovieScenePropertyTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetPropertyPath on a null track"), ELogVerbosity::Error);
		return FString();
	}

	return Track->GetPropertyPath().ToString();
}

FName UMovieScenePropertyTrackExtensions::GetUniqueTrackName(UMovieScenePropertyTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetUniqueTrackName on a null track"), ELogVerbosity::Error);
		return NAME_None;
	}

#if WITH_EDITORONLY_DATA
	return Track->GetTrackName();
#else
	return NAME_None;
#endif
}

void UMovieScenePropertyTrackExtensions::SetObjectPropertyClass(UMovieSceneObjectPropertyTrack* Track, UClass* PropertyClass, bool bInClassProperty)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetObjectPropertyClass on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->Modify();
	Track->PropertyClass = PropertyClass;
#if WITH_EDITORONLY_DATA
	Track->bClassProperty = bInClassProperty;
#endif
}

UClass* UMovieScenePropertyTrackExtensions::GetObjectPropertyClass(UMovieSceneObjectPropertyTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddEventRepeaterSection on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	return Track->PropertyClass;
}

void UMovieScenePropertyTrackExtensions::SetByteTrackEnum(UMovieSceneByteTrack* Track, UEnum* InEnum)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetByteTrackEnum on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->Modify();
	Track->SetEnum(InEnum);
}

UEnum* UMovieScenePropertyTrackExtensions::GetByteTrackEnum(UMovieSceneByteTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetByteTrackEnum on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	return Track->GetEnum();
}



