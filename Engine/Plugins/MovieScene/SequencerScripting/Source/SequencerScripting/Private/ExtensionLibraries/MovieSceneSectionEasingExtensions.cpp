// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneSectionEasingExtensions.h"
#include "MovieSceneSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSectionEasingExtensions)

int32 UMovieSceneSectionEasingExtensions::GetEaseInDuration(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get ease-in duration on a null section"), ELogVerbosity::Error);
		return 0;
	}
	return Section->Easing.GetEaseInDuration();
}

int32 UMovieSceneSectionEasingExtensions::GetEaseOutDuration(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get ease-out duration on a null section"), ELogVerbosity::Error);
		return 0;
	}
	return Section->Easing.GetEaseOutDuration();
}

void UMovieSceneSectionEasingExtensions::SetEaseInDuration(UMovieSceneSection* Section, int32 InDuration)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot set ease-in duration on a null section"), ELogVerbosity::Error);
		return;
	}
	Section->Modify();
	Section->Easing.bManualEaseIn = true;
	Section->Easing.ManualEaseInDuration = InDuration;
}

void UMovieSceneSectionEasingExtensions::SetEaseOutDuration(UMovieSceneSection* Section, int32 InDuration)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot set ease-out duration on a null section"), ELogVerbosity::Error);
		return;
	}
	Section->Modify();
	Section->Easing.bManualEaseOut = true;
	Section->Easing.ManualEaseOutDuration = InDuration;
}
