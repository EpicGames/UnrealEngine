// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneSectionExtensions.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "KeysAndChannels/MovieSceneScriptingBool.h"
#include "Evaluation/MovieSceneTimeTransform.h"
#include "KeysAndChannels/MovieSceneScriptingByte.h"
#include "KeysAndChannels/MovieSceneScriptingInteger.h"
#include "KeysAndChannels/MovieSceneScriptingFloat.h"
#include "KeysAndChannels/MovieSceneScriptingDouble.h"
#include "KeysAndChannels/MovieSceneScriptingString.h"
#include "KeysAndChannels/MovieSceneScriptingText.h"
#include "KeysAndChannels/MovieSceneScriptingEvent.h"
#include "KeysAndChannels/MovieSceneScriptingActorReference.h"
#include "KeysAndChannels/MovieSceneScriptingObjectPath.h"
#include "KeysAndChannels/MovieSceneScriptingParticle.h"
#include "Sections/MovieSceneSubSection.h"
#include "Trace/Trace.inl"
#include "UObject/Package.h" // IWYU pragma: keep

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSectionExtensions)


bool UMovieSceneSectionExtensions::HasStartFrame(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call HasStartFrame on a null section"), ELogVerbosity::Error);
		return false;
	}

	return Section->HasStartFrame();
}

int32 UMovieSceneSectionExtensions::GetStartFrame(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetStartFrame on a null section"), ELogVerbosity::Error);
		return -1;
	}

	if (!Section->HasStartFrame())
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have a start frame"), ELogVerbosity::Error);
		return -1;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		return ConvertFrameTime(UE::MovieScene::DiscreteInclusiveLower(Section->GetRange()), MovieScene->GetTickResolution(), DisplayRate).FloorToFrame().Value;
	}
	else
	{
		return -1;
	}
}

float UMovieSceneSectionExtensions::GetStartFrameSeconds(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetStartFrameSeconds on a null section"), ELogVerbosity::Error);
		return -1.f;
	}

	if (!Section->HasStartFrame())
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have a start frame"), ELogVerbosity::Error);
		return -1.f;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		return DisplayRate.AsSeconds(ConvertFrameTime(UE::MovieScene::DiscreteInclusiveLower(Section->GetRange()), MovieScene->GetTickResolution(), DisplayRate));
	}

	return -1.f;
}

bool UMovieSceneSectionExtensions::HasEndFrame(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call HasEndFrame on a null section"), ELogVerbosity::Error);
		return false;
	}

	return Section->HasEndFrame();
}


int32 UMovieSceneSectionExtensions::GetEndFrame(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetEndFrame on a null section"), ELogVerbosity::Error);
		return -1;
	}

	if (!Section->HasEndFrame())
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have an end frame"), ELogVerbosity::Error);
		return -1;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		return ConvertFrameTime(UE::MovieScene::DiscreteExclusiveUpper(Section->GetRange()), MovieScene->GetTickResolution(), DisplayRate).FloorToFrame().Value;
	}
	else
	{
		return -1;
	}
}

float UMovieSceneSectionExtensions::GetEndFrameSeconds(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetEndFrameSeconds on a null section"), ELogVerbosity::Error);
		return -1.f;
	}

	if (!Section->HasEndFrame())
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have an end frame"), ELogVerbosity::Error);
		return -1.f;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		return DisplayRate.AsSeconds(ConvertFrameTime(UE::MovieScene::DiscreteExclusiveUpper(Section->GetRange()), MovieScene->GetTickResolution(), DisplayRate));
	}

	return -1.f;
}

bool UMovieSceneSectionExtensions::GetAutoSizeHasStartFrame(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetAutoSizeHasStartFrame on a null section"), ELogVerbosity::Error);
		return false;
	}

	TOptional<TRange<FFrameNumber>> AutoSizeRange = Section->GetAutoSizeRange();
	return AutoSizeRange.IsSet() && AutoSizeRange->GetLowerBound().IsClosed();
}

int32 UMovieSceneSectionExtensions::GetAutoSizeStartFrame(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetAutoSizeStartFrame on a null section"), ELogVerbosity::Error);
		return -1;
	}

	TOptional<TRange<FFrameNumber>> AutoSizeRange = Section->GetAutoSizeRange();
	if (!AutoSizeRange.IsSet() || AutoSizeRange->GetLowerBound().IsOpen())
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have an AutoSize start frame"), ELogVerbosity::Error);
		return -1;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		return ConvertFrameTime(UE::MovieScene::DiscreteInclusiveLower(AutoSizeRange.GetValue()), MovieScene->GetTickResolution(), DisplayRate).FloorToFrame().Value;
	}

	return -1;
}

float UMovieSceneSectionExtensions::GetAutoSizeStartFrameSeconds(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetAutoSizeStartFrameSeconds on a null section"), ELogVerbosity::Error);
		return -1.f;
	}

	TOptional<TRange<FFrameNumber>> AutoSizeRange = Section->GetAutoSizeRange();
	if (!AutoSizeRange.IsSet() || AutoSizeRange->GetLowerBound().IsOpen())
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have an AutoSize start frame"), ELogVerbosity::Error);
		return -1.f;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		return DisplayRate.AsSeconds(ConvertFrameTime(UE::MovieScene::DiscreteInclusiveLower(AutoSizeRange.GetValue()), MovieScene->GetTickResolution(), DisplayRate));
	}

	return -1.f;
}

bool UMovieSceneSectionExtensions::GetAutoSizeHasEndFrame(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetAutoSizeHasEndFrame on a null section"), ELogVerbosity::Error);
		return false;
	}

	TOptional<TRange<FFrameNumber>> AutoSizeRange = Section->GetAutoSizeRange();
	return AutoSizeRange.IsSet() && AutoSizeRange->GetUpperBound().IsClosed();
}

int32 UMovieSceneSectionExtensions::GetAutoSizeEndFrame(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetAutoSizeEndFrame on a null section"), ELogVerbosity::Error);
		return -1;
	}

	TOptional<TRange<FFrameNumber>> AutoSizeRange = Section->GetAutoSizeRange();
	if (!AutoSizeRange.IsSet() || AutoSizeRange->GetUpperBound().IsOpen())
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have an AutoSize end frame"), ELogVerbosity::Error);
		return -1;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		return ConvertFrameTime(UE::MovieScene::DiscreteExclusiveUpper(AutoSizeRange.GetValue()), MovieScene->GetTickResolution(), DisplayRate).FloorToFrame().Value;
	}

	return -1;
}

float UMovieSceneSectionExtensions::GetAutoSizeEndFrameSeconds(UMovieSceneSection* Section)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetAutoSizeEndFrameSeconds on a null section"), ELogVerbosity::Error);
		return -1.f;
	}

	TOptional<TRange<FFrameNumber>> AutoSizeRange = Section->GetAutoSizeRange();
	if (!AutoSizeRange.IsSet() || AutoSizeRange->GetUpperBound().IsOpen())
	{
		FFrame::KismetExecutionMessage(TEXT("Section does not have an AutoSize end frame"), ELogVerbosity::Error);
		return -1.f;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		return DisplayRate.AsSeconds(ConvertFrameTime(UE::MovieScene::DiscreteExclusiveUpper(AutoSizeRange.GetValue()), MovieScene->GetTickResolution(), DisplayRate));
	}

	return -1.f;
}

void UMovieSceneSectionExtensions::SetRange(UMovieSceneSection* Section, int32 StartFrame, int32 EndFrame)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetRange on a null section"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		TRange<FFrameNumber> NewRange;
		NewRange.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(ConvertFrameTime(StartFrame, DisplayRate, MovieScene->GetTickResolution()).FrameNumber));
		NewRange.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(ConvertFrameTime(EndFrame, DisplayRate, MovieScene->GetTickResolution()).FrameNumber));

		if (NewRange.GetLowerBound().IsOpen() || NewRange.GetUpperBound().IsOpen() || NewRange.GetLowerBoundValue() <= NewRange.GetUpperBoundValue())
		{
			Section->SetRange(NewRange);
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Invalid range specified"), ELogVerbosity::Error);
		}
	}
}

void UMovieSceneSectionExtensions::SetRangeSeconds(UMovieSceneSection* Section, float StartTime, float EndTime)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetRangeSeconds on a null section"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		TRange<FFrameNumber> NewRange;
		NewRange.SetLowerBound((StartTime * MovieScene->GetTickResolution()).RoundToFrame());
		NewRange.SetUpperBound((EndTime * MovieScene->GetTickResolution()).RoundToFrame());

		if (NewRange.GetLowerBound().IsOpen() || NewRange.GetUpperBound().IsOpen() || NewRange.GetLowerBoundValue() <= NewRange.GetUpperBoundValue())
		{
			Section->SetRange(NewRange);
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Invalid range specified"), ELogVerbosity::Error);
		}
	}
}

void UMovieSceneSectionExtensions::SetStartFrame(UMovieSceneSection* Section, int32 StartFrame)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetStartFrame on a null section"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		Section->SetStartFrame(ConvertFrameTime(StartFrame, DisplayRate, MovieScene->GetTickResolution()).FrameNumber);
	}
}

void UMovieSceneSectionExtensions::SetStartFrameSeconds(UMovieSceneSection* Section, float StartTime)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetStartFrameSeconds on a null section"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		Section->SetStartFrame((StartTime * MovieScene->GetTickResolution()).RoundToFrame());
	}
}

void UMovieSceneSectionExtensions::SetStartFrameBounded(UMovieSceneSection* Section, bool bIsBounded)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetStartFrameBounded on a null section"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		if (bIsBounded)
		{
			int32 NewFrameNumber = 0;
			if (!MovieScene->GetPlaybackRange().GetLowerBound().IsOpen())
			{
				NewFrameNumber = MovieScene->GetPlaybackRange().GetLowerBoundValue().Value;
			}

			Section->SectionRange.Value.SetLowerBound(TRangeBound<FFrameNumber>(FFrameNumber(NewFrameNumber)));
		}
		else
		{
			Section->SectionRange.Value.SetLowerBound(TRangeBound<FFrameNumber>());
		}
	}
}

void UMovieSceneSectionExtensions::SetEndFrame(UMovieSceneSection* Section, int32 EndFrame)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetEndFrame on a null section"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		Section->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive(ConvertFrameTime(EndFrame, DisplayRate, MovieScene->GetTickResolution()).FrameNumber));
	}
}

void UMovieSceneSectionExtensions::SetEndFrameSeconds(UMovieSceneSection* Section, float EndTime)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetEndFrameSeconds on a null section"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		Section->SetEndFrame(TRangeBound<FFrameNumber>::Exclusive((EndTime * MovieScene->GetTickResolution()).RoundToFrame()));
	}
}

void UMovieSceneSectionExtensions::SetEndFrameBounded(UMovieSceneSection* Section, bool bIsBounded)
{
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetEndFrameBounded on a null section"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Section->GetTypedOuter<UMovieScene>();
	if (MovieScene)
	{
		if (bIsBounded)
		{
			int32 NewFrameNumber = 0;
			if (!MovieScene->GetPlaybackRange().GetUpperBound().IsOpen())
			{
				NewFrameNumber = MovieScene->GetPlaybackRange().GetUpperBoundValue().Value;
			}

			Section->SectionRange.Value.SetUpperBound(TRangeBound<FFrameNumber>(FFrameNumber(NewFrameNumber)));
		}
		else
		{
			Section->SectionRange.Value.SetUpperBound(TRangeBound<FFrameNumber>());
		}
	}
}

template<>
void SetScriptingChannelHandle<FMovieSceneFloatChannel, UMovieSceneScriptingFloatChannel>(UMovieSceneScriptingFloatChannel* ScriptingChannel, FMovieSceneChannelProxy& ChannelProxy, int32 ChannelIndex)
{
	ScriptingChannel->FloatChannelHandle = ChannelProxy.MakeHandle<FMovieSceneFloatChannel>(ChannelIndex);
}
template<>
void SetScriptingChannelHandle<FMovieSceneDoubleChannel, UMovieSceneScriptingFloatChannel>(UMovieSceneScriptingFloatChannel* ScriptingChannel, FMovieSceneChannelProxy& ChannelProxy, int32 ChannelIndex)
{
	ScriptingChannel->DoubleChannelHandle = ChannelProxy.MakeHandle<FMovieSceneDoubleChannel>(ChannelIndex);
}



TMap<FGuid,TFunction<bool(UMovieSceneSection* Section, TSubclassOf<UMovieSceneScriptingChannel> ChannelType, TArray<UMovieSceneScriptingChannel*>& OutChannels)>>  UMovieSceneSectionExtensions::CustomChannelsCallbacks;

FGuid UMovieSceneSectionExtensions::SetGetCustomChannelsFunction(TFunction<bool(UMovieSceneSection* Section, TSubclassOf<UMovieSceneScriptingChannel> ChannelType, TArray<UMovieSceneScriptingChannel*>& OutChannels) > CustommFunc)
{
	FGuid NewGuid = FGuid::NewGuid();
	CustomChannelsCallbacks.Add(NewGuid,CustommFunc);
	return NewGuid;
}
void UMovieSceneSectionExtensions::RemoveGetCustomChannelsFunction(const FGuid& InGuid)
{
	CustomChannelsCallbacks.Remove(InGuid);
}

TArray<UMovieSceneScriptingChannel*> UMovieSceneSectionExtensions::GetAllChannels(UMovieSceneSection* Section)
{
	TArray<UMovieSceneScriptingChannel*> Channels;
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get channels for null section"), ELogVerbosity::Error);
		return Channels;
	}
	FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();

	// ToDo: This isn't a great way of collecting all of the channel types and instantiating the correct UObject for it
	// but given that we're already hard-coding the UObject implementations... We currently support the following channel
	// types: Bool, Byte, Float, Double, Integer, String, Text, Event, ActorReference
	TWeakObjectPtr<UMovieSceneSequence> Sequence = Section->GetTypedOuter<UMovieSceneSequence>();
	GetScriptingChannelsForChannel<FMovieSceneBoolChannel, UMovieSceneScriptingBoolChannel>(ChannelProxy, Sequence, Section, Channels);
	GetScriptingChannelsForChannel<FMovieSceneByteChannel, UMovieSceneScriptingByteChannel>(ChannelProxy, Sequence, Section, Channels);
	GetScriptingChannelsForChannel<FMovieSceneFloatChannel, UMovieSceneScriptingFloatChannel>(ChannelProxy, Sequence, Section, Channels);
	GetScriptingChannelsForChannel<FMovieSceneDoubleChannel, UMovieSceneScriptingDoubleChannel>(ChannelProxy, Sequence, Section, Channels);
	GetScriptingChannelsForChannel<FMovieSceneIntegerChannel, UMovieSceneScriptingIntegerChannel>(ChannelProxy, Sequence, Section, Channels);
	GetScriptingChannelsForChannel<FMovieSceneStringChannel, UMovieSceneScriptingStringChannel>(ChannelProxy, Sequence, Section, Channels);
	GetScriptingChannelsForChannel<FMovieSceneTextChannel, UMovieSceneScriptingTextChannel>(ChannelProxy, Sequence, Section, Channels);
	GetScriptingChannelsForChannel<FMovieSceneEventChannel, UMovieSceneScriptingEventChannel>(ChannelProxy, Sequence, Section, Channels);
	GetScriptingChannelsForChannel<FMovieSceneActorReferenceChannel, UMovieSceneScriptingActorReferenceChannel>(ChannelProxy, Sequence, Section, Channels);
	GetScriptingChannelsForChannel<FMovieSceneObjectPathChannel, UMovieSceneScriptingObjectPathChannel>(ChannelProxy, Sequence, Section, Channels);
	GetScriptingChannelsForChannel<FMovieSceneParticleChannel, UMovieSceneScriptingParticleChannel>(ChannelProxy, Sequence, Section, Channels);
	

	for (const TPair<FGuid,TFunction<bool(UMovieSceneSection* Section, TSubclassOf<UMovieSceneScriptingChannel> ChannelType, TArray<UMovieSceneScriptingChannel*>& OutChannels)>>&
		Function : CustomChannelsCallbacks)
	{
		Function.Value(Section, UMovieSceneScriptingChannel::StaticClass(), Channels);
	}
	return Channels;
}

TArray<UMovieSceneScriptingChannel*> UMovieSceneSectionExtensions::GetChannelsByType(UMovieSceneSection* Section, TSubclassOf<UMovieSceneScriptingChannel> ChannelType)
{
	TArray<UMovieSceneScriptingChannel*> Channels;
	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get channels for null section"), ELogVerbosity::Error);
		return Channels;
	}
 
	FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
 
	// ToDo: This needs to be done dynamically in the future but there's not a good way to do that right now with all of the SFINAE-based Templating driving these channels
	TWeakObjectPtr<UMovieSceneSequence> Sequence = Section->GetTypedOuter<UMovieSceneSequence>();

	//But we added this to be able to specify custom UMoiveSceneScriptingChanngels
	for (const TPair<FGuid, TFunction<bool(UMovieSceneSection* Section, TSubclassOf<UMovieSceneScriptingChannel> ChannelType, TArray<UMovieSceneScriptingChannel*>& OutChannels)>>&
		Function : CustomChannelsCallbacks)
	{
		if (Function.Value(Section, ChannelType, Channels))
		{
			return Channels;
		}
	}



	if (ChannelType == UMovieSceneScriptingBoolChannel::StaticClass())					{ GetScriptingChannelsForChannel<FMovieSceneBoolChannel, UMovieSceneScriptingBoolChannel>(ChannelProxy, Sequence, Section, Channels); }
	else if (ChannelType == UMovieSceneScriptingByteChannel::StaticClass())				{ GetScriptingChannelsForChannel<FMovieSceneByteChannel, UMovieSceneScriptingByteChannel>(ChannelProxy, Sequence, Section, Channels); }
	else if (ChannelType == UMovieSceneScriptingFloatChannel::StaticClass())			{ GetScriptingChannelsForChannel<FMovieSceneFloatChannel, UMovieSceneScriptingFloatChannel>(ChannelProxy, Sequence, Section, Channels); }
	else if (ChannelType == UMovieSceneScriptingDoubleChannel::StaticClass())			{ GetScriptingChannelsForChannel<FMovieSceneDoubleChannel, UMovieSceneScriptingDoubleChannel>(ChannelProxy, Sequence, Section, Channels); }
	else if (ChannelType == UMovieSceneScriptingIntegerChannel::StaticClass())			{ GetScriptingChannelsForChannel<FMovieSceneIntegerChannel, UMovieSceneScriptingIntegerChannel>(ChannelProxy, Sequence, Section, Channels); }
	else if (ChannelType == UMovieSceneScriptingStringChannel::StaticClass())			{ GetScriptingChannelsForChannel<FMovieSceneStringChannel, UMovieSceneScriptingStringChannel>(ChannelProxy, Sequence, Section, Channels); }
	else if (ChannelType == UMovieSceneScriptingTextChannel::StaticClass())				{ GetScriptingChannelsForChannel<FMovieSceneTextChannel, UMovieSceneScriptingTextChannel>(ChannelProxy, Sequence, Section, Channels); }	
	else if (ChannelType == UMovieSceneScriptingEventChannel::StaticClass())			{ GetScriptingChannelsForChannel<FMovieSceneEventChannel, UMovieSceneScriptingEventChannel>(ChannelProxy, Sequence, Section, Channels); }
	else if (ChannelType == UMovieSceneScriptingActorReferenceChannel::StaticClass()) { GetScriptingChannelsForChannel<FMovieSceneActorReferenceChannel, UMovieSceneScriptingActorReferenceChannel>(ChannelProxy, Sequence, Section, Channels); }
	else if (ChannelType == UMovieSceneScriptingObjectPathChannel::StaticClass()) { GetScriptingChannelsForChannel<FMovieSceneObjectPathChannel, UMovieSceneScriptingObjectPathChannel>(ChannelProxy, Sequence, Section, Channels); }
	else if (ChannelType == UMovieSceneScriptingParticleChannel::StaticClass()) { GetScriptingChannelsForChannel<FMovieSceneParticleChannel, UMovieSceneScriptingParticleChannel>(ChannelProxy, Sequence, Section, Channels); }
	else
	{
		FFrame::KismetExecutionMessage(TEXT("Unsupported ChannelType for FindChannelsByType!"), ELogVerbosity::Error);
	}

	return Channels;
}

UMovieSceneScriptingChannel* UMovieSceneSectionExtensions::GetChannel(UMovieSceneSection* Section, const FName& ChannelName)
{
#if WITH_EDITORONLY_DATA

	if (!Section)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get channel for null section"), ELogVerbosity::Error);
		return nullptr;
	}
	FMovieSceneChannel* MovieSceneChannel =  UMovieSceneSectionExtensions::GetMovieSceneChannel(Section, ChannelName);
	if (!MovieSceneChannel)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot get channel with this name"), ELogVerbosity::Error);
		return nullptr;
	}

	FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
	FMovieSceneChannelHandle ChanneHandle = ChannelProxy.GetChannelByName(ChannelName);
	TWeakObjectPtr<UMovieSceneSequence> Sequence = Section->GetTypedOuter<UMovieSceneSequence>();
	if (ChanneHandle.GetChannelTypeName() == FMovieSceneBoolChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneBoolChannel, UMovieSceneScriptingBoolChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	} 
	else if (ChanneHandle.GetChannelTypeName() == FMovieSceneByteChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneByteChannel, UMovieSceneScriptingByteChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	}
	else if (ChanneHandle.GetChannelTypeName() == FMovieSceneFloatChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneFloatChannel, UMovieSceneScriptingFloatChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	}
	else if (ChanneHandle.GetChannelTypeName() == FMovieSceneDoubleChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneDoubleChannel, UMovieSceneScriptingDoubleChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	}
	else if (ChanneHandle.GetChannelTypeName() == FMovieSceneIntegerChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneIntegerChannel, UMovieSceneScriptingIntegerChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	}
	else if (ChanneHandle.GetChannelTypeName() == FMovieSceneStringChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneStringChannel, UMovieSceneScriptingStringChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	}
	else if (ChanneHandle.GetChannelTypeName() == FMovieSceneTextChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneTextChannel, UMovieSceneScriptingTextChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	}
	else if (ChanneHandle.GetChannelTypeName() == FMovieSceneEventChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneEventChannel, UMovieSceneScriptingEventChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	}
	else if (ChanneHandle.GetChannelTypeName() == FMovieSceneActorReferenceChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneActorReferenceChannel, UMovieSceneScriptingActorReferenceChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	}
	else if (ChanneHandle.GetChannelTypeName() == FMovieSceneObjectPathChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneObjectPathChannel, UMovieSceneScriptingObjectPathChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	}
	else if (ChanneHandle.GetChannelTypeName() == FMovieSceneParticleChannel::StaticStruct()->GetFName())
	{
		return GetScriptingChannelForChannel<FMovieSceneParticleChannel, UMovieSceneScriptingParticleChannel>(MovieSceneChannel, ChannelProxy, ChanneHandle, Sequence, Section, ChannelName);
	}
#endif
	FFrame::KismetExecutionMessage(TEXT("Could not find channel"), ELogVerbosity::Error);
	return nullptr;
}

bool GetSubSectionChain(UMovieSceneSubSection* InSubSection, UMovieSceneSequence* ParentSequence, TArray<UMovieSceneSubSection*>& SubSectionChain)
{
	UMovieScene* ParentMovieScene = ParentSequence->GetMovieScene();
	for (UMovieSceneTrack* Track : ParentMovieScene->GetTracks())
	{
		for (UMovieSceneSection* Section : Track->GetAllSections())
		{
			if (Section == InSubSection)
			{
				SubSectionChain.Add(InSubSection);
				return true;
			}
			if (Section->IsA(UMovieSceneSubSection::StaticClass()))
			{
				UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
				if (GetSubSectionChain(InSubSection, SubSection->GetSequence(), SubSectionChain))
				{
					SubSectionChain.Add(SubSection);
				}
			}
		}
	}
	return false;
}


int32 UMovieSceneSectionExtensions::GetParentSequenceFrame(UMovieSceneSubSection* InSubSection, int32 InFrame, UMovieSceneSequence* ParentSequence)
{
	if (InSubSection == nullptr || ParentSequence == nullptr)
	{
		UE_LOGF(LogMovieScene, Error, "UMovieSceneSectionExtension::GetParentSequenceFrame failed because either sub section or parent sequence is null! SubSection: %ls ParentSequence: %ls", InSubSection ? *InSubSection->GetFullName() : TEXT("None"), ParentSequence ? *ParentSequence->GetFullName() : TEXT("None"));
		return InFrame;
	}

	TArray<UMovieSceneSubSection*> SubSectionChain;
	GetSubSectionChain(InSubSection, ParentSequence, SubSectionChain);

	FFrameRate LocalDisplayRate = InSubSection->GetSequence()->GetMovieScene()->GetDisplayRate();
	FFrameRate LocalTickResolution = InSubSection->GetSequence()->GetMovieScene()->GetTickResolution();
	FFrameTime LocalFrameTime = ConvertFrameTime(InFrame, LocalDisplayRate, LocalTickResolution);

	for (int32 SectionIndex = 0; SectionIndex < SubSectionChain.Num(); ++SectionIndex)
	{
		TOptional<FFrameTime> NewLocalTime = SubSectionChain[SectionIndex]->OuterToInnerTransform().Inverse().TryTransformTime(LocalFrameTime);
		if (NewLocalTime)
		{
			LocalFrameTime = NewLocalTime.GetValue();
		}
	}

	FFrameRate ParentDisplayRate = ParentSequence->GetMovieScene()->GetDisplayRate();
	FFrameRate ParentTickResolution = ParentSequence->GetMovieScene()->GetTickResolution();

	LocalFrameTime = ConvertFrameTime(LocalFrameTime, ParentTickResolution, ParentDisplayRate);
	return LocalFrameTime.GetFrame().Value;
}

FMovieSceneChannel* UMovieSceneSectionExtensions::GetMovieSceneChannel(UMovieSceneSection* Section, const FName& Name)
{
#if WITH_EDITORONLY_DATA

	FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
	for (const FMovieSceneChannelEntry& Entry : Section->GetChannelProxy().GetAllEntries())
	{
		TArrayView<FMovieSceneChannel* const>  Channels = Entry.GetChannels();
		TArrayView<const FMovieSceneChannelMetaData> MetaData = Entry.GetMetaData();
		for (int32 Index = 0; Index < MetaData.Num(); ++Index)
		{
			if (MetaData[Index].bEnabled)
			{
				const FName ChannelName = MetaData[Index].Name;
				if (ChannelName == Name)
				{
					return Channels[Index];
				}
			}
		}
	}
#endif
	return nullptr;
}
