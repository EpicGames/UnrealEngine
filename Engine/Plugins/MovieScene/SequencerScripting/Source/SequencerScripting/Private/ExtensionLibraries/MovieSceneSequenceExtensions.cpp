// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneSequenceExtensions.h"
#include "MovieSceneBindingProxy.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "MovieScenePossessable.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "MovieSceneFolder.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "MovieSceneSpawnable.h"
#include "SequencerScriptingRange.h"
#include "MovieSceneBindingReferences.h"
#include "Bindings/MovieSceneSpawnableActorBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSequenceExtensions)

TArray<UMovieSceneTrack*> UMovieSceneSequenceExtensions::FilterTracks(TArrayView<UMovieSceneTrack* const> InTracks, UClass* DesiredClass, bool bExactMatch)
{
	TArray<UMovieSceneTrack*> Tracks;

	for (UMovieSceneTrack* Track : InTracks)
	{
		UClass* TrackClass = Track->GetClass();

		if ( TrackClass == DesiredClass || (!bExactMatch && TrackClass->IsChildOf(DesiredClass)) )
		{
			Tracks.Add(Track);
		}
	}

	return Tracks;
}

UMovieScene* UMovieSceneSequenceExtensions::GetMovieScene(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetMovieScene on a null sequence"), ELogVerbosity::Error);
		return nullptr;
	}

	return Sequence->GetMovieScene();
}

TArray<UMovieSceneTrack*> UMovieSceneSequenceExtensions::GetTracks(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetTracks on a null sequence"), ELogVerbosity::Error);
		return TArray<UMovieSceneTrack*>();
	}

	TArray<UMovieSceneTrack*> Tracks;

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		Tracks = MovieScene->GetTracks();

		if (UMovieSceneTrack* CameraCutTrack = MovieScene->GetCameraCutTrack())
		{
			Tracks.Add(CameraCutTrack);
		}
	}

	return Tracks;
}

TArray<UMovieSceneTrack*> UMovieSceneSequenceExtensions::FindTracksByType(UMovieSceneSequence* Sequence, TSubclassOf<UMovieSceneTrack> TrackType)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call FindTracksByType on a null sequence"), ELogVerbosity::Error);
		return TArray<UMovieSceneTrack*>();
	}

	UMovieScene* MovieScene   = GetMovieScene(Sequence);
	UClass*      DesiredClass = TrackType.Get();

	if (MovieScene && DesiredClass)
	{
		bool bExactMatch = false;
		TArray<UMovieSceneTrack*> MatchedTracks = FilterTracks(MovieScene->GetTracks(), TrackType.Get(), bExactMatch);

		// Have to check camera cut tracks separately since they're not in the root tracks array (why?)
		UMovieSceneTrack* CameraCutTrack = MovieScene->GetCameraCutTrack();
		if (CameraCutTrack && CameraCutTrack->GetClass()->IsChildOf(DesiredClass))
		{
			MatchedTracks.Add(CameraCutTrack);
		}

		return MatchedTracks;
	}

	return TArray<UMovieSceneTrack*>();
}

TArray<UMovieSceneTrack*> UMovieSceneSequenceExtensions::FindTracksByExactType(UMovieSceneSequence* Sequence, TSubclassOf<UMovieSceneTrack> TrackType)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call FindTracksByExactType on a null sequence"), ELogVerbosity::Error);
		return TArray<UMovieSceneTrack*>();
	}

	UMovieScene* MovieScene   = GetMovieScene(Sequence);
	UClass*      DesiredClass = TrackType.Get();

	if (MovieScene && DesiredClass)
	{
		bool bExactMatch = true;
		TArray<UMovieSceneTrack*> MatchedTracks = FilterTracks(MovieScene->GetTracks(), TrackType.Get(), bExactMatch);

		// Have to check camera cut tracks separately since they're not in the root tracks array (why?)
		UMovieSceneTrack* CameraCutTrack = MovieScene->GetCameraCutTrack();
		if (CameraCutTrack && CameraCutTrack->GetClass() == DesiredClass)
		{
			MatchedTracks.Add(CameraCutTrack);
		}

		return MatchedTracks;
	}

	return TArray<UMovieSceneTrack*>();
}

UMovieSceneTrack* UMovieSceneSequenceExtensions::AddTrack(UMovieSceneSequence* Sequence, TSubclassOf<UMovieSceneTrack> TrackType)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddTrack on a null sequence"), ELogVerbosity::Error);
		return nullptr;
	}

	// @todo: sequencer-python: track type compatibility with sequence. Currently that's really only loosely defined by track editors, which is not sufficient here.
	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		if (TrackType->IsChildOf(UMovieSceneCameraCutTrack::StaticClass()))
		{
			return MovieScene->AddCameraCutTrack(TrackType);
		}
		else
		{
			return MovieScene->AddTrack(TrackType);
		}
	}

	return nullptr;
}

bool UMovieSceneSequenceExtensions::RemoveTrack(UMovieSceneSequence* Sequence, UMovieSceneTrack* Track)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveTrack on a null sequence"), ELogVerbosity::Error);
		return false;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		if (Track->IsA(UMovieSceneCameraCutTrack::StaticClass()))
		{
			MovieScene->RemoveCameraCutTrack();
			return true;
		}
		else
		{
			return MovieScene->RemoveTrack(*Track);
		}
	}

	return false;
}


FFrameRate UMovieSceneSequenceExtensions::GetDisplayRate(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetDisplayRate on a null sequence"), ELogVerbosity::Error);
		return FFrameRate();
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	return MovieScene ? MovieScene->GetDisplayRate() : FFrameRate();
}

void UMovieSceneSequenceExtensions::SetDisplayRate(UMovieSceneSequence* Sequence, FFrameRate DisplayRate)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetDisplayRate on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		MovieScene->Modify();

		MovieScene->SetDisplayRate(DisplayRate);
	}
}

FFrameRate UMovieSceneSequenceExtensions::GetTickResolution(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetTickResolution on a null sequence"), ELogVerbosity::Error);
		return FFrameRate();
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	return MovieScene ? MovieScene->GetTickResolution() : FFrameRate();
}

void UMovieSceneSequenceExtensions::SetTickResolution(UMovieSceneSequence* Sequence, FFrameRate TickResolution)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetTickResolution on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		MovieScene->Modify();

		UE::MovieScene::TimeHelpers::MigrateFrameTimes(MovieScene->GetTickResolution(), TickResolution, MovieScene);
	}
}

void UMovieSceneSequenceExtensions::SetTickResolutionDirectly(UMovieSceneSequence* Sequence, FFrameRate TickResolution)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetTickResolutionDirectly on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)	
	{
		MovieScene->Modify();

		MovieScene->SetTickResolutionDirectly(TickResolution);
	}
}

FSequencerScriptingRange UMovieSceneSequenceExtensions::MakeRange(UMovieSceneSequence* Sequence, int32 StartFrame, int32 Duration)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call MakeRange on a null sequence"), ELogVerbosity::Error);
		return FSequencerScriptingRange();
	}

	FFrameRate FrameRate = GetDisplayRate(Sequence);
	return FSequencerScriptingRange::FromNative(TRange<FFrameNumber>(StartFrame, StartFrame+Duration), FrameRate, FrameRate);
}

FSequencerScriptingRange UMovieSceneSequenceExtensions::MakeRangeSeconds(UMovieSceneSequence* Sequence, float StartTime, float Duration)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call MakeRangeSeconds on a null sequence"), ELogVerbosity::Error);
		return FSequencerScriptingRange();
	}

	FFrameRate FrameRate = GetDisplayRate(Sequence);
	return FSequencerScriptingRange::FromNative(TRange<FFrameNumber>((StartTime*FrameRate).FloorToFrame(), ((StartTime+Duration) * FrameRate).CeilToFrame()), FrameRate, FrameRate);
}

FSequencerScriptingRange UMovieSceneSequenceExtensions::GetPlaybackRange(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetPlaybackRange on a null sequence"), ELogVerbosity::Error);
		return FSequencerScriptingRange();
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		return FSequencerScriptingRange::FromNative(MovieScene->GetPlaybackRange(), GetTickResolution(Sequence), GetDisplayRate(Sequence));
	}
	else
	{
		return FSequencerScriptingRange();
	}
}

int32 UMovieSceneSequenceExtensions::GetPlaybackStart(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetPlaybackStart on a null sequence"), ELogVerbosity::Error);
		return -1;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		FFrameRate DisplayRate = GetDisplayRate(Sequence);
		return ConvertFrameTime(UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange()), GetTickResolution(Sequence), DisplayRate).FloorToFrame().Value;
	}
	else
	{
		return -1;
	}
}

float UMovieSceneSequenceExtensions::GetPlaybackStartSeconds(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetPlaybackStartSeconds on a null sequence"), ELogVerbosity::Error);
		return -1.f;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		FFrameRate DisplayRate = GetDisplayRate(Sequence);
		return DisplayRate.AsSeconds(ConvertFrameTime(UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange()), GetTickResolution(Sequence), DisplayRate));
	}

	return -1.f;
}

int32 UMovieSceneSequenceExtensions::GetPlaybackEnd(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetPlaybackEnd on a null sequence"), ELogVerbosity::Error);
		return -1;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		FFrameRate DisplayRate = GetDisplayRate(Sequence);
		return ConvertFrameTime(UE::MovieScene::DiscreteExclusiveUpper(MovieScene->GetPlaybackRange()), GetTickResolution(Sequence), DisplayRate).FloorToFrame().Value;
	}
	else
	{
		return -1;
	}
}

float UMovieSceneSequenceExtensions::GetPlaybackEndSeconds(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetPlaybackEndSeconds on a null sequence"), ELogVerbosity::Error);
		return -1.f;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		FFrameRate DisplayRate = GetDisplayRate(Sequence);
		return DisplayRate.AsSeconds(ConvertFrameTime(UE::MovieScene::DiscreteExclusiveUpper(MovieScene->GetPlaybackRange()), GetTickResolution(Sequence), DisplayRate));
	}

	return -1.f;
}

void UMovieSceneSequenceExtensions::SetPlaybackStart(UMovieSceneSequence* Sequence, int32 StartFrame)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetPlaybackStart on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		FFrameRate DisplayRate = GetDisplayRate(Sequence);

		TRange<FFrameNumber> NewRange = MovieScene->GetPlaybackRange();
		NewRange.SetLowerBoundValue(ConvertFrameTime(StartFrame, DisplayRate, GetTickResolution(Sequence)).FrameNumber);

		MovieScene->SetPlaybackRange(NewRange);
	}
}

void UMovieSceneSequenceExtensions::SetPlaybackStartSeconds(UMovieSceneSequence* Sequence, float StartTime)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetPlaybackStartSeconds on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		TRange<FFrameNumber> NewRange = MovieScene->GetPlaybackRange();
		NewRange.SetLowerBoundValue((StartTime * GetTickResolution(Sequence)).RoundToFrame());

		MovieScene->SetPlaybackRange(NewRange);
	}
}

void UMovieSceneSequenceExtensions::SetPlaybackEnd(UMovieSceneSequence* Sequence, int32 EndFrame)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetPlaybackEnd on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		FFrameRate DisplayRate = GetDisplayRate(Sequence);

		TRange<FFrameNumber> NewRange = MovieScene->GetPlaybackRange();
		NewRange.SetUpperBoundValue(ConvertFrameTime(EndFrame, DisplayRate, GetTickResolution(Sequence)).FrameNumber);

		MovieScene->SetPlaybackRange(NewRange);
	}
}

void UMovieSceneSequenceExtensions::SetPlaybackEndSeconds(UMovieSceneSequence* Sequence, float EndTime)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetPlaybackEndSeconds on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		TRange<FFrameNumber> NewRange = MovieScene->GetPlaybackRange();
		NewRange.SetUpperBoundValue((EndTime * GetTickResolution(Sequence)).RoundToFrame());

		MovieScene->SetPlaybackRange(NewRange);
	}
}

void UMovieSceneSequenceExtensions::SetViewRangeStart(UMovieSceneSequence* Sequence, double StartTimeInSeconds)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetViewRangeStart on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
#if WITH_EDITORONLY_DATA
		MovieScene->SetViewRange(StartTimeInSeconds, MovieScene->GetEditorData().ViewEnd);
#endif
	}
}

double UMovieSceneSequenceExtensions::GetViewRangeStart(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetViewRangeStart on a null sequence"), ELogVerbosity::Error);
		return 0;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
#if WITH_EDITORONLY_DATA
		return MovieScene->GetEditorData().ViewStart;
#endif
	}
	return 0;
}

void UMovieSceneSequenceExtensions::SetViewRangeEnd(UMovieSceneSequence* Sequence, double EndTimeInSeconds)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetViewRangeEnd on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
#if WITH_EDITORONLY_DATA
		MovieScene->SetViewRange(MovieScene->GetEditorData().ViewStart, EndTimeInSeconds);
#endif
	}
}

double UMovieSceneSequenceExtensions::GetViewRangeEnd(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetViewRangeEnd on a null sequence"), ELogVerbosity::Error);
		return 0;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
#if WITH_EDITORONLY_DATA
		return MovieScene->GetEditorData().ViewEnd;
#endif
	}
	return 0;
}

void UMovieSceneSequenceExtensions::SetWorkRangeStart(UMovieSceneSequence* Sequence, double StartTimeInSeconds)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetWorkRangeStart on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
#if WITH_EDITORONLY_DATA
		MovieScene->SetWorkingRange(StartTimeInSeconds, MovieScene->GetEditorData().WorkEnd);
#endif
	}
}

double UMovieSceneSequenceExtensions::GetWorkRangeStart(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetWorkRangeStart on a null sequence"), ELogVerbosity::Error);
		return 0;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
#if WITH_EDITORONLY_DATA
		return MovieScene->GetEditorData().WorkStart;
#endif
	}
	return 0;
}

void UMovieSceneSequenceExtensions::SetWorkRangeEnd(UMovieSceneSequence* Sequence, double EndTimeInSeconds)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetWorkRangeEnd on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
#if WITH_EDITORONLY_DATA
		MovieScene->SetWorkingRange(MovieScene->GetEditorData().WorkStart, EndTimeInSeconds);
#endif
	}
}

double UMovieSceneSequenceExtensions::GetWorkRangeEnd(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetWorkRangeEnd on a null sequence"), ELogVerbosity::Error);
		return 0;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
#if WITH_EDITORONLY_DATA
		return MovieScene->GetEditorData().WorkEnd;
#endif
	}
	return 0;
}

void UMovieSceneSequenceExtensions::SetEvaluationType(UMovieSceneSequence* Sequence, EMovieSceneEvaluationType InEvaluationType)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetEvaluationType on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		MovieScene->SetEvaluationType(InEvaluationType);
	}
}

EMovieSceneEvaluationType UMovieSceneSequenceExtensions::GetEvaluationType(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetEvaluationType on a null sequence"), ELogVerbosity::Error);
		return EMovieSceneEvaluationType::WithSubFrames;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		return MovieScene->GetEvaluationType();
	}

	return EMovieSceneEvaluationType::WithSubFrames;
}

void UMovieSceneSequenceExtensions::SetClockSource(UMovieSceneSequence* Sequence, EUpdateClockSource InClockSource)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetClockSource on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		MovieScene->SetClockSource(InClockSource);
	}
}

EUpdateClockSource UMovieSceneSequenceExtensions::GetClockSource(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetClockSource on a null sequence"), ELogVerbosity::Error);
		return EUpdateClockSource::Tick;
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		return MovieScene->GetClockSource();
	}

	return EUpdateClockSource::Tick;
}

UMovieSceneClock* UMovieSceneSequenceExtensions::GetCustomClock(UMovieSceneSequence* InSequence)
{
	if (!InSequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetCustomClock on a null sequence"), ELogVerbosity::Error);
		return nullptr;
	}

	UMovieScene* MovieScene = GetMovieScene(InSequence);
	if (MovieScene)
	{
		return MovieScene->GetCustomClock();
	}

	return nullptr;
}

FMovieSceneBindingProxy UMovieSceneSequenceExtensions::FindBindingByName(UMovieSceneSequence* Sequence, FString Name)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call FindBindingByName on a null sequence"), ELogVerbosity::Error);
		return FMovieSceneBindingProxy();
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
		{
			const FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
			if (Possessable.GetName() == Name)
			{
				return FMovieSceneBindingProxy(Possessable.GetGuid(), Sequence);
			}
		}

		for (int32 Index = 0; Index < MovieScene->GetSpawnableCount(); ++Index)
		{
			const FMovieSceneSpawnable& Spawnable = MovieScene->GetSpawnable(Index);
			if (Spawnable.GetName() == Name)
			{
				return FMovieSceneBindingProxy(Spawnable.GetGuid(), Sequence);
			}
		}
	}
	return FMovieSceneBindingProxy();
}

FMovieSceneBindingProxy UMovieSceneSequenceExtensions::FindBindingById(UMovieSceneSequence* Sequence, FGuid BindingId)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call FindBindingById on a null sequence"), ELogVerbosity::Error);
		return FMovieSceneBindingProxy();
	}

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		const FMovieSceneBinding* Binding = MovieScene->FindBinding(BindingId);
		if (Binding)
		{
			return FMovieSceneBindingProxy(Binding->GetObjectGuid(), Sequence);
		}
	}
	return FMovieSceneBindingProxy();
}

TArray<FMovieSceneBindingProxy> UMovieSceneSequenceExtensions::GetBindings(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetBindings on a null sequence"), ELogVerbosity::Error);
		return TArray<FMovieSceneBindingProxy>();
	}

	TArray<FMovieSceneBindingProxy> AllBindings;

	const UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		for (const FMovieSceneBinding& Binding : MovieScene->GetBindings())
		{
			AllBindings.Emplace(Binding.GetObjectGuid(), Sequence);
		}
	}

	return AllBindings;
}

TArray<FMovieSceneBindingProxy> UMovieSceneSequenceExtensions::GetSpawnables(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetSpawnables on a null sequence"), ELogVerbosity::Error);
		return TArray<FMovieSceneBindingProxy>();
	}

	TArray<FMovieSceneBindingProxy> AllSpawnables;

	UMovieScene* MovieScene = GetMovieScene(Sequence);
	if (MovieScene)
	{
		int32 Count = MovieScene->GetSpawnableCount();
		AllSpawnables.Reserve(Count);
		for (int32 i=0; i < Count; ++i)
		{
			AllSpawnables.Emplace(MovieScene->GetSpawnable(i).GetGuid(), Sequence);
		}
	}

	if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
	{
		for (const FMovieSceneBindingReference& BindingReference : BindingReferences->GetAllReferences())
		{
			if (BindingReference.CustomBinding && BindingReference.CustomBinding->IsA<UMovieSceneSpawnableActorBinding>())
			{
				AllSpawnables.Emplace(BindingReference.ID, Sequence);
			}
		}
	}

	return AllSpawnables;
}

TArray<FMovieSceneBindingProxy> UMovieSceneSequenceExtensions::GetPossessables(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetPossessables on a null sequence"), ELogVerbosity::Error);
		return TArray<FMovieSceneBindingProxy>();
	}

	TArray<FMovieSceneBindingProxy> AllPossessables;

	if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
	{
		for (const FMovieSceneBindingReference& BindingReference : BindingReferences->GetAllReferences())
		{
			if (!BindingReference.CustomBinding)
			{
				AllPossessables.Emplace(BindingReference.ID, Sequence);
			}
		}
	}
	else
	{
		UMovieScene* MovieScene = GetMovieScene(Sequence);
		if (MovieScene)
		{
			int32 Count = MovieScene->GetPossessableCount();
			for (int32 i = 0; i < Count; ++i)
			{
				AllPossessables.Emplace(MovieScene->GetPossessable(i).GetGuid(), Sequence);
			}
		}
	}

	return AllPossessables;
}

FMovieSceneBindingProxy UMovieSceneSequenceExtensions::AddPossessable(UMovieSceneSequence* Sequence, UObject* ObjectToPossess)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddPossessable on a null sequence"), ELogVerbosity::Error);
		return FMovieSceneBindingProxy();
	}

	if (!ObjectToPossess)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddPossessable on a null object"), ELogVerbosity::Error);
		return FMovieSceneBindingProxy();
	}

	FGuid NewGuid = Sequence->CreatePossessable(ObjectToPossess);
	return FMovieSceneBindingProxy(NewGuid, Sequence);
}

FMovieSceneBindingProxy UMovieSceneSequenceExtensions::AddSpawnableFromInstance(UMovieSceneSequence* Sequence, UObject* ObjectToSpawn)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddSpawnableFromInstance on a null sequence"), ELogVerbosity::Error);
		return FMovieSceneBindingProxy();
	}

	if (!ObjectToSpawn)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddSpawnableFromInstance on a null object"), ELogVerbosity::Error);
		return FMovieSceneBindingProxy();
	}

	FGuid NewGuid = Sequence->AllowsSpawnableObjects() ? Sequence->CreateSpawnable(ObjectToSpawn) : FGuid();
	return FMovieSceneBindingProxy(NewGuid, Sequence);
}

FMovieSceneBindingProxy UMovieSceneSequenceExtensions::AddSpawnableFromClass(UMovieSceneSequence* Sequence, UClass* ClassToSpawn)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddSpawnableFromClass on a null sequence"), ELogVerbosity::Error);
		return FMovieSceneBindingProxy();
	}

	if (!ClassToSpawn)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddSpawnableFromClass on a null class"), ELogVerbosity::Error);
		return FMovieSceneBindingProxy();
	}

	FGuid NewGuid = Sequence->AllowsSpawnableObjects() ? Sequence->CreateSpawnable(ClassToSpawn) : FGuid();
	return FMovieSceneBindingProxy(NewGuid, Sequence);
}

TArray<UObject*> UMovieSceneSequenceExtensions::LocateBoundObjects(UMovieSceneSequence* Sequence, const FMovieSceneBindingProxy& InBinding, UObject* Context)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call LocateBoundObjects on a null sequence"), ELogVerbosity::Error);
		return TArray<UObject*>();
	}

	using namespace UE::MovieScene;

	FSharedPlaybackStateCreateParams CreateParams;
	CreateParams.PlaybackContext = Context;
	TSharedRef<FSharedPlaybackState> TransientPlaybackState = MakeShared<FSharedPlaybackState>(*Sequence, CreateParams);

	FMovieSceneEvaluationState State;
	TransientPlaybackState->AddCapabilityRaw(&State);
	State.AssignSequence(MovieSceneSequenceID::Root, *Sequence, TransientPlaybackState);

	TArrayView<TWeakObjectPtr<>> Objects = State.FindBoundObjects(InBinding.BindingID, MovieSceneSequenceID::Root, TransientPlaybackState);

	TArray<UObject*> Result;
	for (TWeakObjectPtr<> WeakObject : Objects)
	{
		if (WeakObject.IsValid())
		{
			Result.Add(WeakObject.Get());
		}
	}
	return Result;
}

FMovieSceneObjectBindingID UMovieSceneSequenceExtensions::GetBindingID(UMovieSceneSequence* Sequence, const FMovieSceneBindingProxy& InBinding)
{
	return UE::MovieScene::FRelativeObjectBindingID(InBinding.BindingID);
}

FMovieSceneObjectBindingID UMovieSceneSequenceExtensions::GetPortableBindingID(UMovieSceneSequence* RootSequence, UMovieSceneSequence* DestinationSequence, const FMovieSceneBindingProxy& InBinding)
{
	if (!RootSequence || !DestinationSequence || !InBinding.Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid sequence specified."), ELogVerbosity::Error);
		return FMovieSceneObjectBindingID();
	}

	// If they are all the same sequence, we're dealing with a local binding - this requires no computation
	if (RootSequence == DestinationSequence && RootSequence == InBinding.Sequence)
	{
		return UE::MovieScene::FRelativeObjectBindingID(InBinding.BindingID);
	}

	// Destination is the destination for the BindingID to be serialized / resolved within
	// Target is the target sequence that contains the actual binding

	TOptional<FMovieSceneSequenceID> DestinationSequenceID;
	TOptional<FMovieSceneSequenceID> TargetSequenceID;

	if (RootSequence == DestinationSequence)
	{
		DestinationSequenceID = MovieSceneSequenceID::Root;
	}
	if (RootSequence == InBinding.Sequence)
	{
		TargetSequenceID = MovieSceneSequenceID::Root;
	}

	// We know that we have at least one sequence ID to find, otherwise we would have entered the ::Local branch above
	UMovieSceneCompiledDataManager*     CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();
	const FMovieSceneCompiledDataID     DataID              = CompiledDataManager->Compile(RootSequence);
	const FMovieSceneSequenceHierarchy* Hierarchy           = CompiledDataManager->FindHierarchy(DataID);

	// If we have no hierarchy, the supplied RootSequence does not have any sub-sequences so the callee has given us bogus parameters
	if (!Hierarchy)
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Root Sequence ('%s') does not have any sub-sequences."), *RootSequence->GetPathName()), ELogVerbosity::Error);
		return FMovieSceneObjectBindingID();
	}

	// Find the destination and/or target sequence IDs as required.
	// This method is flawed if there is more than one instance of the sequence within the hierarchy
	// In this case we just pick the first one we find
	for (const TTuple<FMovieSceneSequenceID, FMovieSceneSubSequenceData>& Pair : Hierarchy->AllSubSequenceData())
	{
		if (UMovieSceneSequence* SubSequence = Pair.Value.GetSequence())
		{
			if (!TargetSequenceID.IsSet() && SubSequence == InBinding.Sequence)
			{
				TargetSequenceID = Pair.Key;
			}
			if (!DestinationSequenceID.IsSet() && SubSequence == DestinationSequence)
			{
				DestinationSequenceID = Pair.Key;
			}

			if (DestinationSequenceID.IsSet() && TargetSequenceID.IsSet())
			{
				break;
			}
		}
	}

	if (!DestinationSequenceID.IsSet())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Unable to locate DestinationSequence ('%s') within Root Sequence hierarchy ('%s')."), *DestinationSequence->GetPathName(), *RootSequence->GetPathName()), ELogVerbosity::Error);
		return FMovieSceneObjectBindingID();
	}

	if (!TargetSequenceID.IsSet())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("Unable to locate Sequence for InBinding ('%s') within Root Sequence hierarchy ('%s')."), *InBinding.Sequence->GetPathName(), *RootSequence->GetPathName()), ELogVerbosity::Error);
		return FMovieSceneObjectBindingID();
	}

	return UE::MovieScene::FRelativeObjectBindingID(DestinationSequenceID.GetValue(), TargetSequenceID.GetValue(), InBinding.BindingID, Hierarchy);
}

FMovieSceneBindingProxy UMovieSceneSequenceExtensions::ResolveBindingID(UMovieSceneSequence* RootSequence, FMovieSceneObjectBindingID InObjectBindingID)
{
	if (!RootSequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call ResolveBindingID on a null sequence"), ELogVerbosity::Error);
		return FMovieSceneBindingProxy();
	}

	UMovieSceneSequence* Sequence = RootSequence;

	FMovieSceneCompiledDataID DataID = UMovieSceneCompiledDataManager::GetPrecompiledData()->Compile(RootSequence);

	const FMovieSceneSequenceHierarchy* Hierarchy = UMovieSceneCompiledDataManager::GetPrecompiledData()->FindHierarchy(DataID);
	if (Hierarchy)
	{
		if (UMovieSceneSequence* SubSequence = Hierarchy->FindSubSequence(InObjectBindingID.ResolveSequenceID(MovieSceneSequenceID::Root, Hierarchy)))
		{
			Sequence = SubSequence;
		}
	}

	return FMovieSceneBindingProxy(InObjectBindingID.GetGuid(), Sequence);
}

TArray<UMovieSceneFolder*> UMovieSceneSequenceExtensions::GetRootFoldersInSequence(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetRootFoldersInSequence on a null sequence"), ELogVerbosity::Error);
		return TArray<UMovieSceneFolder*>();
	}

	TArray<UMovieSceneFolder*> Result;

#if WITH_EDITORONLY_DATA
	UMovieScene* Scene = Sequence->GetMovieScene();
	if (Scene)
	{
		Result = Scene->GetRootFolders();
	}
#endif

	return Result;
}

UMovieSceneFolder* UMovieSceneSequenceExtensions::AddRootFolderToSequence(UMovieSceneSequence* Sequence, FString NewFolderName)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddRootFolderToSequence on a null sequence"), ELogVerbosity::Error);
		return nullptr;
	}

	UMovieSceneFolder* NewFolder = nullptr;
	
#if WITH_EDITORONLY_DATA
	
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		MovieScene->Modify();
		NewFolder = NewObject<UMovieSceneFolder>(MovieScene, NAME_None, RF_Transactional);
		NewFolder->SetFolderName(FName(*NewFolderName));
		MovieScene->AddRootFolder(NewFolder);
	}
#endif

	return NewFolder;
}

void UMovieSceneSequenceExtensions::RemoveRootFolderFromSequence(UMovieSceneSequence* Sequence, UMovieSceneFolder* Folder)
{
	if (!Sequence || !Folder)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveRootFolderFromSequence on a null sequence or folder"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* FolderMovieScene = Folder->GetTypedOuter<UMovieScene>();
	if (FolderMovieScene != Sequence->GetMovieScene())
	{
		FFrame::KismetExecutionMessage(*FString::Printf(TEXT("The folder '%s' does not belong to sequence '%s'"),
			*Folder->GetFolderName().ToString(), *Sequence->GetName()), ELogVerbosity::Error);
		return;
	}

#if WITH_EDITORONLY_DATA
	if (FolderMovieScene)
	{
		FolderMovieScene->Modify();
		const int32 NumFoldersRemoved = FolderMovieScene->RemoveRootFolder(Folder);
		if (NumFoldersRemoved == 0)
		{
			FFrame::KismetExecutionMessage(*FString::Printf(TEXT("The specified folder '%s' is not a root folder"),
				*Folder->GetFolderName().ToString()), ELogVerbosity::Error);
		}
	}
#endif
}

TArray<FMovieSceneMarkedFrame> UMovieSceneSequenceExtensions::GetMarkedFramesFromSequence(UMovieSceneSequence* Sequence, EMovieSceneTimeUnit TimeUnit)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetMarkedFrames on a null sequence"), ELogVerbosity::Error);
		return TArray<FMovieSceneMarkedFrame>();
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		TArray<FMovieSceneMarkedFrame> MarkedFrames = MovieScene->GetMarkedFrames();
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			FFrameRate TickResolution = MovieScene->GetTickResolution();
			FFrameRate DisplayRate = MovieScene->GetDisplayRate();

			for (FMovieSceneMarkedFrame& MarkedFrame : MarkedFrames)
			{
				MarkedFrame.FrameNumber = FFrameRate::TransformTime(MarkedFrame.FrameNumber, TickResolution, DisplayRate).RoundToFrame();
			}
		}

		return MarkedFrames;
	}

	return TArray<FMovieSceneMarkedFrame>();
}

int32 UMovieSceneSequenceExtensions::AddMarkedFrameToSequence(UMovieSceneSequence* Sequence, const FMovieSceneMarkedFrame& InMarkedFrame, EMovieSceneTimeUnit TimeUnit)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddMarkedFrame on a null sequence"), ELogVerbosity::Error);
		return INDEX_NONE;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		MovieScene->Modify();

		FMovieSceneMarkedFrame MarkedFrame(InMarkedFrame);
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			MarkedFrame.FrameNumber = FFrameRate::TransformTime(MarkedFrame.FrameNumber, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution()).RoundToFrame();
		}

		return MovieScene->AddMarkedFrame(MarkedFrame);
	}
	return INDEX_NONE;
}


void UMovieSceneSequenceExtensions::SetMarkedFrameInSequence(UMovieSceneSequence* Sequence, int32 InMarkIndex, FFrameNumber InFrameNumber, EMovieSceneTimeUnit TimeUnit)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetMarkedFrame on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			InFrameNumber = FFrameRate::TransformTime(InFrameNumber, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution()).RoundToFrame();
		}

		return MovieScene->SetMarkedFrame(InMarkIndex, InFrameNumber);
	}
}


void UMovieSceneSequenceExtensions::DeleteMarkedFrame(UMovieSceneSequence* Sequence, int32 DeleteIndex)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call DeleteMarkedFrame on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		MovieScene->Modify();

		MovieScene->DeleteMarkedFrame(DeleteIndex);
	}
}

void UMovieSceneSequenceExtensions::DeleteMarkedFrames(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call DeleteMarkedFrames on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		MovieScene->Modify();

		MovieScene->DeleteMarkedFrames();
	}
}

void UMovieSceneSequenceExtensions::SortMarkedFrames(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SortMarkedFrames on a null sequence"), ELogVerbosity::Error);
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		return MovieScene->SortMarkedFrames();
	}
}

int32 UMovieSceneSequenceExtensions::FindMarkedFrameByLabel(UMovieSceneSequence* Sequence, const FString& InLabel)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call FindMarkedFrameByLabel on a null sequence"), ELogVerbosity::Error);
		return INDEX_NONE;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		return MovieScene->FindMarkedFrameByLabel(InLabel);
	}
	return INDEX_NONE;
}

int32 UMovieSceneSequenceExtensions::FindMarkedFrameByFrameNumberInSequence(UMovieSceneSequence* Sequence, FFrameNumber InFrameNumber, EMovieSceneTimeUnit TimeUnit)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call FindMarkedFrameByFrameNumber on a null sequence"), ELogVerbosity::Error);
		return INDEX_NONE;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			InFrameNumber = FFrameRate::TransformTime(InFrameNumber, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution()).RoundToFrame();
		}

		return MovieScene->FindMarkedFrameByFrameNumber(InFrameNumber);
	}
	return INDEX_NONE;
}

int32 UMovieSceneSequenceExtensions::FindNextMarkedFrameInSequence(UMovieSceneSequence* Sequence, FFrameNumber InFrameNumber, bool bForward, EMovieSceneTimeUnit TimeUnit)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call FindNextMarkedFrame on a null sequence"), ELogVerbosity::Error);
		return INDEX_NONE;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			InFrameNumber = FFrameRate::TransformTime(InFrameNumber, MovieScene->GetDisplayRate(), MovieScene->GetTickResolution()).RoundToFrame();
		}

		return MovieScene->FindNextMarkedFrame(InFrameNumber, bForward);
	}
	return INDEX_NONE;
}

void UMovieSceneSequenceExtensions::SetReadOnly(UMovieSceneSequence* Sequence, bool bInReadOnly)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetReadOnly on a null sequence"), ELogVerbosity::Error);
		return;
	}

#if WITH_EDITORONLY_DATA
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		MovieScene->SetReadOnly(bInReadOnly);
	}
#endif
}

bool UMovieSceneSequenceExtensions::IsReadOnly(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call IsReadOnly on a null sequence"), ELogVerbosity::Error);
		return false;
	}

#if WITH_EDITORONLY_DATA
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{

		return MovieScene->IsReadOnly();
	}
#endif

	return false;
}

void UMovieSceneSequenceExtensions::SetPlaybackRangeLocked(UMovieSceneSequence* Sequence, bool bInLocked)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetPlaybackRangeLocked on a null sequence"), ELogVerbosity::Error);
		return;
	}

#if WITH_EDITORONLY_DATA
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		MovieScene->SetPlaybackRangeLocked(bInLocked);
	}
#endif
}

bool UMovieSceneSequenceExtensions::IsPlaybackRangeLocked(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call IsPlaybackRangeLocked on a null sequence"), ELogVerbosity::Error);
		return false;
	}

#if WITH_EDITORONLY_DATA
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{

		return MovieScene->IsPlaybackRangeLocked();
	}
#endif

	return false;
}

void UMovieSceneSequenceExtensions::SetMarkedFramesLocked(UMovieSceneSequence* Sequence, bool bInLocked)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetMarkedFramesLocked on a null sequence"), ELogVerbosity::Error);
		return;
	}

#if WITH_EDITORONLY_DATA
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{
		MovieScene->SetMarkedFramesLocked(bInLocked);
	}
#endif
}

bool UMovieSceneSequenceExtensions::AreMarkedFramesLocked(UMovieSceneSequence* Sequence)
{
	if (!Sequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AreMarkedFramesLocked on a null sequence"), ELogVerbosity::Error);
		return false;
	}

#if WITH_EDITORONLY_DATA
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (MovieScene)
	{

		return MovieScene->AreMarkedFramesLocked();
	}
#endif

	return false;
}

