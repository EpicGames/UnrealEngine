// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneTimeWarpExtensions.h"
#include "Variants/MovieSceneTimeWarpVariant.h"
#include "UObject/Stack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTimeWarpExtensions)


double UMovieSceneTimeWarpExtensions::Conv_TimeWarpVariantToPlayRate(const FMovieSceneTimeWarpVariant& TimeWarp)
{
	if (TimeWarp.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
	{
		return TimeWarp.AsFixedPlayRate();
	}

	FFrame::KismetExecutionMessage(TEXT("The specified TimeWarp is not a fixed play rate"), ELogVerbosity::Error);
	return 1.f;
}

FMovieSceneTimeWarpVariant UMovieSceneTimeWarpExtensions::Conv_PlayRateToTimeWarpVariant(double ConstantPlayRate)
{
	return FMovieSceneTimeWarpVariant(ConstantPlayRate);
}

double UMovieSceneTimeWarpExtensions::ToFixedPlayRate(const FMovieSceneTimeWarpVariant& TimeWarp)
{
	return Conv_TimeWarpVariantToPlayRate(TimeWarp);
}

void UMovieSceneTimeWarpExtensions::SetFixedPlayRate(FMovieSceneTimeWarpVariant& TimeWarp, double FixedPlayRate)
{
	TimeWarp.Set(FixedPlayRate);
}

void UMovieSceneTimeWarpExtensions::BreakTimeWarp(const FMovieSceneTimeWarpVariant& TimeWarp, double& FixedPlayRate)
{
	if (TimeWarp.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
	{
		FixedPlayRate = TimeWarp.AsFixedPlayRate();
	}
}

FMovieSceneTimeWarpVariant UMovieSceneTimeWarpExtensions::MakeTimeWarp(double FixedPlayRate)
{
	return FMovieSceneTimeWarpVariant(FixedPlayRate);
}