// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneTrackExtensions.h"
#include "MovieSceneNameableTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTrackExtensions)

void UMovieSceneTrackExtensions::SetDisplayName(UMovieSceneTrack* Track, const FText& InName)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetDisplayName on a null track"), ELogVerbosity::Error);
		return;
	}

	if (UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(Track))
	{
#if WITH_EDITORONLY_DATA
		NameableTrack->SetDisplayName(InName);
#endif
	}
}

FText UMovieSceneTrackExtensions::GetDisplayName(UMovieSceneTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetDisplayName on a null track"), ELogVerbosity::Error);
		return FText::GetEmpty();
	}

#if WITH_EDITORONLY_DATA
	return Track->GetDisplayName();
#else
	return FText::GetEmpty();
#endif
}

void UMovieSceneTrackExtensions::SetTrackRowDisplayName(UMovieSceneTrack* Track, const FText& InName, int32 RowIndex)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetTrackRowDisplayName on a null track"), ELogVerbosity::Error);
		return;
	}

	if (UMovieSceneNameableTrack* NameableTrack = Cast<UMovieSceneNameableTrack>(Track))
	{
#if WITH_EDITORONLY_DATA
		NameableTrack->SetTrackRowDisplayName(InName, RowIndex);
#endif
	}
}

FText UMovieSceneTrackExtensions::GetTrackRowDisplayName(UMovieSceneTrack* Track, int32 RowIndex)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetTrackRowDisplayName on a null track"), ELogVerbosity::Error);
		return FText::GetEmpty();
	}

#if WITH_EDITORONLY_DATA
	return Track->GetTrackRowDisplayName(RowIndex);
#else
	return FText::GetEmpty();
#endif
}

UMovieSceneSection* UMovieSceneTrackExtensions::AddSection(UMovieSceneTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call AddSection on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	UMovieSceneSection* NewSection = Track->CreateNewSection();

	if (NewSection)
	{
		Track->Modify();

		Track->AddSection(*NewSection);
	}

	return NewSection;
}

TArray<UMovieSceneSection*> UMovieSceneTrackExtensions::GetSections(UMovieSceneTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetSections on a null track"), ELogVerbosity::Error);
		return TArray<UMovieSceneSection*>();
	}

	return Track->GetAllSections();
}

void UMovieSceneTrackExtensions::RemoveSection(UMovieSceneTrack* Track, UMovieSceneSection* Section)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call RemoveSection on a null track"), ELogVerbosity::Error);
		return;
	}

	if (Section)
	{
		Track->Modify();

		Track->RemoveSection(*Section);
	}
}

int32 UMovieSceneTrackExtensions::GetSortingOrder(UMovieSceneTrack* Track) 
{ 
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetSortingOrder on a null track"), ELogVerbosity::Error);
		return -1;
	}

#if WITH_EDITORONLY_DATA
	return Track->GetSortingOrder(); 
#else
	return 0;
#endif
}
 
void UMovieSceneTrackExtensions::SetSortingOrder(UMovieSceneTrack* Track, int32 SortingOrder) 
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetSortingOrder on a null track"), ELogVerbosity::Error);
		return;
	}

#if WITH_EDITORONLY_DATA
	Track->SetSortingOrder(SortingOrder); 
#endif
}

FColor UMovieSceneTrackExtensions::GetColorTint(UMovieSceneTrack* Track) 
{ 
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetColorTint on a null track"), ELogVerbosity::Error);
		return FColor();
	}

#if WITH_EDITORONLY_DATA
	return Track->GetColorTint(); 
#else
	return FColor();
#endif
}

void UMovieSceneTrackExtensions::SetColorTint(UMovieSceneTrack* Track, const FColor& ColorTint) 
{ 
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetColorTint on a null track"), ELogVerbosity::Error);
		return;
	}

#if WITH_EDITORONLY_DATA
	Track->SetColorTint(ColorTint); 
#endif
}

UMovieSceneSection* UMovieSceneTrackExtensions::GetSectionToKey(UMovieSceneTrack* Track) 
{ 
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetSectionToKey on a null track"), ELogVerbosity::Error);
		return nullptr;
	}

	return Track->GetSectionToKey(); 
}

void UMovieSceneTrackExtensions::SetSectionToKey(UMovieSceneTrack* Track, UMovieSceneSection* Section) 
{ 
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetSectionToKey on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->SetSectionToKey(Section); 
}



