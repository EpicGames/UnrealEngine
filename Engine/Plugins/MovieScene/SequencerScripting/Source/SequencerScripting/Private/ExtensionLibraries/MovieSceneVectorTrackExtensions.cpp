// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/MovieSceneVectorTrackExtensions.h"
#include "Tracks/MovieSceneVectorTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneVectorTrackExtensions)

void UMovieSceneFloatVectorTrackExtensions::SetNumChannelsUsed(UMovieSceneFloatVectorTrack* Track, int32 InNumChannelsUsed)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetNumChannelsUsed on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->Modify();

	Track->SetNumChannelsUsed(InNumChannelsUsed);
}


int32 UMovieSceneFloatVectorTrackExtensions::GetNumChannelsUsed(UMovieSceneFloatVectorTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetNumChannelsUsed on a null track"), ELogVerbosity::Error);
		return -1;
	}

	return Track->GetNumChannelsUsed();
}

void UMovieSceneDoubleVectorTrackExtensions::SetNumChannelsUsed(UMovieSceneDoubleVectorTrack* Track, int32 InNumChannelsUsed)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call SetNumChannelsUsed on a null track"), ELogVerbosity::Error);
		return;
	}

	Track->Modify();

	Track->SetNumChannelsUsed(InNumChannelsUsed);
}


int32 UMovieSceneDoubleVectorTrackExtensions::GetNumChannelsUsed(UMovieSceneDoubleVectorTrack* Track)
{
	if (!Track)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot call GetNumChannelsUsed on a null track"), ELogVerbosity::Error);
		return -1;
	}

	return Track->GetNumChannelsUsed();
}


