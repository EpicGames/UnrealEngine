// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionLibraries/SequencerScriptingRangeExtensions.h"
#include "SequencerScriptingRange.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerScriptingRangeExtensions)

bool USequencerScriptingRangeExtensions::HasStart(const FSequencerScriptingRange& Range)
{
	return Range.bHasStart;
}

bool USequencerScriptingRangeExtensions::HasEnd(const FSequencerScriptingRange& Range)
{
	return Range.bHasEnd;
}

void USequencerScriptingRangeExtensions::RemoveStart(FSequencerScriptingRange& Range)
{
	Range.bHasStart = false;
	Range.InclusiveStart = TNumericLimits<int32>::Lowest();
}

void USequencerScriptingRangeExtensions::RemoveEnd(FSequencerScriptingRange& Range)
{
	Range.bHasEnd = false;
	Range.ExclusiveEnd = TNumericLimits<int32>::Max();
}

float USequencerScriptingRangeExtensions::GetStartSeconds(const FSequencerScriptingRange& Range)
{
	return FFrameTime(Range.InclusiveStart) / Range.InternalRate;
}

float USequencerScriptingRangeExtensions::GetEndSeconds(const FSequencerScriptingRange& Range)
{
	return FFrameTime(Range.ExclusiveEnd) / Range.InternalRate;
}

void USequencerScriptingRangeExtensions::SetStartSeconds(FSequencerScriptingRange& InRange, float Start)
{
	InRange.bHasStart = true;
	InRange.InclusiveStart = (Start * InRange.InternalRate).FloorToFrame().Value;
}

void USequencerScriptingRangeExtensions::SetEndSeconds(FSequencerScriptingRange& InRange, float End)
{
	InRange.bHasEnd = true;
	InRange.ExclusiveEnd = (End * InRange.InternalRate).CeilToFrame().Value;
}

int32 USequencerScriptingRangeExtensions::GetStartFrame(const FSequencerScriptingRange& Range)
{
	return Range.InclusiveStart;
}

int32 USequencerScriptingRangeExtensions::GetEndFrame(const FSequencerScriptingRange& Range)
{
	return Range.ExclusiveEnd;
}

void USequencerScriptingRangeExtensions::SetStartFrame(FSequencerScriptingRange& Range, int32 Start)
{
	Range.bHasStart = true;
	Range.InclusiveStart = Start;
}

void USequencerScriptingRangeExtensions::SetEndFrame(FSequencerScriptingRange& Range, int32 End)
{
	Range.bHasEnd = true;
	Range.ExclusiveEnd = End;
}
