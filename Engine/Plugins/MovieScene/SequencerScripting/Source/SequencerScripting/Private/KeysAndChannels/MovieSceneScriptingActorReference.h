// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneScriptingChannel.h"
#include "Sections/MovieSceneActorReferenceSection.h"
#include "KeyParams.h"
#include "MovieSceneTimeUnit.h"

#include "MovieSceneScriptingActorReference.generated.h"

class UMovieSceneScriptingChannel;
class UMovieSceneScriptingKey;
template <typename ChannelType, typename ScriptingKeyType, typename ScriptingKeyValueType> struct TMovieSceneScriptingChannel;

/**
* Exposes a Sequencer Actor Reference type key to Python/Blueprints.
* Stores a reference to the data so changes to this class are forwarded onto the underlying data structures.
*/
UCLASS(BlueprintType)
class UMovieSceneScriptingActorReferenceKey : public UMovieSceneScriptingKey, public TMovieSceneScriptingKey<FMovieSceneActorReferenceChannel, FMovieSceneActorReferenceKey>
{
	GENERATED_BODY()
public:
	/**
	* Gets the time for this key from the owning channel.
	* @param TimeUnit	Should the time be returned in Display Rate frames (possibly with a sub-frame value) or in Tick Resolution with no sub-frame values?
	* @return			The time of this key which combines both the frame number and the sub-frame it is on. Sub-frame will be zero if you request Tick Resolution.	
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Time (Actor Reference)"))
	virtual FFrameTime GetTime(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) const override { return GetTimeFromChannel(KeyHandle, OwningSequence, TimeUnit); }
	
	/**
	* Sets the time for this key in the owning channel. Will replace any key that already exists at that frame number in this channel.
	* @param NewFrameNumber	What frame should this key be moved to? This should be in the time unit specified by TimeUnit.
	* @param SubFrame		If using Display Rate time, what is the sub-frame this should go to? Clamped [0-1), and ignored with when TimeUnit is set to Tick Resolution. 
	* @param TimeUnit		Should the NewFrameNumber be interpreted as Display Rate frames or in Tick Resolution?
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Time (Actor Reference)"))
	void SetTime(const FFrameNumber& NewFrameNumber, float SubFrame = 0.f, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) { SetTimeInChannel(KeyHandle, OwningSequence, OwningSection, NewFrameNumber, TimeUnit, SubFrame); }

	/**
	* Gets the value for this key from the owning channel.
	* @return	The object binding for this key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Value (Actor Reference)"))
	FMovieSceneObjectBindingID GetValue() const
	{
		FMovieSceneActorReferenceKey Value = GetValueFromChannel(KeyHandle);
		return Value.Object;
	}

	/**
	* Sets the value for this key, reflecting it in the owning channel.
	* @param InNewValue	The new object binding for this key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Value (Actor Reference)"))
	void SetValue(const FMovieSceneObjectBindingID& InNewValue)
	{
		FMovieSceneActorReferenceKey ReferenceKey = FMovieSceneActorReferenceKey(InNewValue);
		SetValueInChannel(KeyHandle, OwningSection, ReferenceKey);
	}
};

UCLASS(BlueprintType)
class UMovieSceneScriptingActorReferenceChannel : public UMovieSceneScriptingChannel
{
	GENERATED_BODY()

	using Impl = TMovieSceneScriptingChannel<FMovieSceneActorReferenceChannel, UMovieSceneScriptingActorReferenceKey, FMovieSceneActorReferenceKey>;

public:
	/**
	* Add a key to this channel. This initializes a new key and returns a reference to it.
	* @param	InTime			The frame this key should go on. Respects TimeUnit to determine if it is a display rate frame or a tick resolution frame.
	* @param	NewValue		The value that this key should be created with.
	* @param	SubFrame		Optional [0-1) clamped sub-frame to put this key on. Ignored if TimeUnit is set to Tick Resolution.
	* @param	TimeUnit 		Is the specified InTime in Display Rate frames or Tick Resolution.
	* @return	The key that was created with the specified values at the specified time.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Add Key (Actor Reference)"))
	UMovieSceneScriptingActorReferenceKey* AddKey(const FFrameNumber InTime, FMovieSceneObjectBindingID NewValue, float SubFrame = 0.f, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate)
	{
		FMovieSceneActorReferenceKey ReferenceKey = FMovieSceneActorReferenceKey(NewValue);
		return Impl::AddKeyInChannel(ChannelHandle, OwningSequence, OwningSection, InTime, ReferenceKey, SubFrame, TimeUnit, EMovieSceneKeyInterpolation::Auto);
	}

	/**
	* Removes the specified key. Does nothing if the key is not specified or the key belongs to another channel.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Remove Key (Actor Reference)"))
	virtual void RemoveKey(UMovieSceneScriptingKey* Key)
	{
		Impl::RemoveKeyFromChannel(ChannelHandle, Key);
	}

	/**
	* Gets all of the keys in this channel.
	* @return	An array of UMovieSceneScriptingActorReferenceKeys contained by this channel.
	*			Returns all keys even if clipped by the owning section's boundaries or outside of the current sequence play range.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Keys (Actor Reference)"))
	virtual TArray<UMovieSceneScriptingKey*> GetKeys() const override
	{
		return Impl::GetKeysInChannel(ChannelHandle, OwningSequence, OwningSection);
	}

	/**
	* Gets the keys in this channel specified by the specific index
	* @Indices  The indices from which to get the keys from
	* @return	An array of UMovieSceneScriptingKey's contained by this channel.
	*			Returns all keys specified by the indices, even if out of range.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Keys By Index (Actor Reference)"))
	virtual TArray<UMovieSceneScriptingKey*> GetKeysByIndex(const TArray<int32>& Indices) const override
	{
		return Impl::GetKeysInChannelByIndex(ChannelHandle, OwningSequence, OwningSection, Indices);
	}

	/**
	* Set this channel's default value that should be used when no keys are present.
	* Sets bHasDefaultValue to true automatically.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Default (Actor Reference)"))
	void SetDefault(FMovieSceneObjectBindingID InDefaultValue)
	{
		FMovieSceneActorReferenceKey ReferenceKey = FMovieSceneActorReferenceKey(InDefaultValue);
		Impl::SetDefaultInChannel(ChannelHandle, OwningSequence, OwningSection, ReferenceKey);
	}

	/**
	* Get this channel's default value that will be used when no keys are present. Only a valid
	* value when HasDefault() returns true.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Default (Actor Reference)"))
	FMovieSceneObjectBindingID GetDefault() const
	{
		// FMovieSceneActorReferenceChannel doesn't implement GetDefault via TOptional, so we're wrapping this function by hand as well.
		FMovieSceneActorReferenceChannel* Channel = ChannelHandle.Get();
		if (Channel)
		{
			FMovieSceneActorReferenceKey ReferenceKey = Channel->GetDefault();
			return ReferenceKey.Object;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to get default value."), ELogVerbosity::Error);
		return FMovieSceneObjectBindingID();
	}

	/**
	* Remove this channel's default value causing the channel to have no effect where no keys are present
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Remove Default (Actor Reference)"))
	void RemoveDefault()
	{
		if (OwningSection.IsValid())
		{
			OwningSection->Modify();
		}

		// FMovieSceneActorReferenceChannel doesn't implement RemoveDefault, instead it implements ClearDefault(). Wrapping this function by hand,
		// and not falling back to the template as a result, but keeping the same function name so it is consistent with the other scripting channels.
		FMovieSceneActorReferenceChannel* Channel = ChannelHandle.Get();
		if (Channel)
		{
			Channel->ClearDefault();

#if WITH_EDITOR
			const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
			if (MetaData && OwningSection.IsValid() && OwningSequence.IsValid() && OwningSequence->GetMovieScene())
			{
				TArray<FFrameNumber> Frames;
				OwningSequence->GetMovieScene()->OnChannelChangedWithTime().Broadcast(Frames, MetaData, OwningSection.Get());
			}
#endif
			return;
		}
		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to remove default value."), ELogVerbosity::Error);
	}

	/**
	* @return Does this channel have a default value set?
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Has Default (Actor Reference)"))
	bool HasDefault() const
	{
		return GetDefault() == FMovieSceneObjectBindingID();
	}

	/**
	 * Transform the keys in time in the channel by an offset, scale and pivot
	 * @param	OffsetFrame	    The amount to offset the keys by
	 * @param	Scale	        The amount to scale the keys by
	 * @param	PivotFrame	    The frame to pivot around when scaling the keys
	 * @param   ScriptingRange  The time range of the keys to transform
	 * @param	TimeUnit 		Is the specified OffsetFrame, PivotFrame, and ScriptingRange in Display Rate frames or Tick Resolution.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Transform (Actor Reference)"))
	void Transform(FFrameNumber OffsetFrame, double Scale, FFrameNumber PivotFrame, FSequencerScriptingRange ScriptingRange, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate)
	{
		Impl::TransformKeysInChannel(ChannelHandle, OwningSequence, OwningSection, OffsetFrame, Scale, PivotFrame, ScriptingRange, TimeUnit);
	}

public:
	TWeakObjectPtr<UMovieSceneSequence> OwningSequence;
	TWeakObjectPtr<UMovieSceneSection> OwningSection;
	TMovieSceneChannelHandle<FMovieSceneActorReferenceChannel> ChannelHandle;
};
