// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelData.h"
#include "MovieSceneScriptingChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "KeyParams.h"
#include "MovieSceneTimeUnit.h"

#include "MovieSceneScriptingDouble.generated.h"

class UMovieSceneScriptingChannel;
class UMovieSceneScriptingKey;
enum ERichCurveTangentMode : int;
enum ERichCurveTangentWeightMode : int;
template <typename ChannelType, typename ScriptingKeyType, typename ScriptingKeyValueType> struct TMovieSceneScriptingChannel;

/**
* Exposes a Sequencer double type key to Python/Blueprints.
* Stores a reference to the data so changes to this class are forwarded onto the underlying data structures.
*/
UCLASS(BlueprintType)
class UMovieSceneScriptingDoubleKey : public UMovieSceneScriptingKey, public TMovieSceneScriptingKey<FMovieSceneDoubleChannel, FMovieSceneDoubleValue>
{
	GENERATED_BODY()
public:
	/**
	* Gets the time for this key from the owning channel.
	* @param TimeUnit	Should the time be returned in Display Rate frames (possibly with a sub-frame value) or in Tick Resolution with no sub-frame values?
	* @return			The time of this key which combines both the frame number and the sub-frame it is on. Sub-frame will be zero if you request Tick Resolution.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Time (Double)"))
	virtual FFrameTime GetTime(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) const override { return GetTimeFromChannel(KeyHandle, OwningSequence, TimeUnit); }
	
	/**
	* Sets the time for this key in the owning channel. Will replace any key that already exists at that frame number in this channel.
	* @param NewFrameNumber	What frame should this key be moved to? This should be in the time unit specified by TimeUnit.
	* @param SubFrame		If using Display Rate time, what is the sub-frame this should go to? Clamped [0-1), and ignored with when TimeUnit is set to Tick Resolution.
	* @param TimeUnit		Should the NewFrameNumber be interpreted as Display Rate frames or in Tick Resolution?
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Time (Double)"))
	void SetTime(const FFrameNumber& NewFrameNumber, float SubFrame = 0.f, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) { SetTimeInChannel(KeyHandle, OwningSequence, OwningSection, NewFrameNumber, TimeUnit, SubFrame); }

	/**
	* Gets the value for this key from the owning channel.
	* @return	The double value this key represents.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Value (Double)"))
	double GetValue() const
	{
		return GetValueFromChannel(KeyHandle).Value;
	}

	/**
	* Sets the value for this key, reflecting it in the owning channel.
	* @param InNewValue	The new double value for this key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Value (Double)"))
	void SetValue(double InNewValue)
	{
		FMovieSceneDoubleValue ExistValue = GetValueFromChannel(KeyHandle);
		ExistValue.Value = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistValue);
	}

	/**
	* Gets the interpolation mode for this key from the owning channel.
	* @return	Interpolation mode this key uses to interpolate between this key and the next.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	ERichCurveInterpMode GetInterpolationMode() const
	{
		return GetValueFromChannel(KeyHandle).InterpMode;
	}

	/**
	* Sets the interpolation mode for this key, reflecting it in the owning channel.
	* @param InNewValue	Interpolation mode this key should use to interpolate between this key and the next.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetInterpolationMode(ERichCurveInterpMode InNewValue)
	{
		FMovieSceneDoubleValue ExistValue = GetValueFromChannel(KeyHandle);
		ExistValue.InterpMode = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistValue);
	}

	/**
	* Gets the tangent mode for this key from the owning channel.
	* @return	Tangent mode that this key is using specifying which tangent values are respected when evaluating.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	ERichCurveTangentMode GetTangentMode() const
	{
		return GetValueFromChannel(KeyHandle).TangentMode;
	}

	/**
	* Sets the tangent mode for this key, reflecting it in the owning channel.
	* @param InNewValue	Tangent mode that this key should use to specify which tangent values are respected when evaluating. See ERichCurveTangentMode for more details.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetTangentMode(ERichCurveTangentMode InNewValue)
	{
		FMovieSceneDoubleValue ExistValue = GetValueFromChannel(KeyHandle);
		ExistValue.TangentMode = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistValue);
	}

	/**
	* If Interpolation Mode is RCIM_Cubic, the arriving tangent at this key
	* @return Arrival Tangent value. Represents the geometric tangents in the form of "tan(y/x)" where y is the key's value and x is the seconds (both relative to key)
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	float GetArriveTangent() const
	{
		return GetValueFromChannel(KeyHandle).Tangent.ArriveTangent;
	}

	/**
	* If Interpolation Mode is RCIM_Cubic, the arriving tangent at this key.
	* @param InNewValue	Represents the geometric tangents in the form of "tan(y/x)" where y is the key's value and x is the seconds (both relative to key)
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetArriveTangent(float InNewValue)
	{
		FMovieSceneDoubleValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.ArriveTangent = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}

	/**
	* If Interpolation Mode is RCIM_Cubic, the leaving tangent at this key
	* @return Leaving Tangent value. Represents the geometric tangents in the form of "tan(y/x)" where y is the key's value and x is the seconds (both relative to key)
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	float GetLeaveTangent() const
	{
		return GetValueFromChannel(KeyHandle).Tangent.LeaveTangent;
	}

	/**
	* If Interpolation Mode is RCIM_Cubic, the leaving tangent at this key.
	* @param InNewValue	Represents the geometric tangents in the form of "tan(y/x)" where y is the key's value and x is the seconds (both relative to key)
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetLeaveTangent(float InNewValue)
	{
		FMovieSceneDoubleValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.LeaveTangent = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}

	/**
	* If Interpolation Mode is RCIM_Cubic, the tangent weight mode at this key
	* @return Tangent Weight Mode. See ERichCurveTangentWeightMode for more detail on what each mode does.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	ERichCurveTangentWeightMode GetTangentWeightMode() const
	{
		return GetValueFromChannel(KeyHandle).Tangent.TangentWeightMode;
	}

	/**
	* If Interpolation Mode is RCIM_Cubic, the tangent weight mode at this key.
	* @param InNewValue	Specifies which tangent weights should be respected when evaluating the key. 
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetTangentWeightMode(ERichCurveTangentWeightMode InNewValue)
	{
		FMovieSceneDoubleValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.TangentWeightMode = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}

	/**
	* If Tangent Weight Mode is RCTWM_WeightedArrive or RCTWM_WeightedBoth, the weight of the arriving tangent on the left side.
	* @return Tangent Weight. Represents the length of the hypotenuse in the form of "sqrt(x*x+y*y)" using the same definitions for x and y as tangents.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	float GetArriveTangentWeight() const
	{
		return GetValueFromChannel(KeyHandle).Tangent.ArriveTangentWeight;
	}

	/**
	* If Tangent Weight Mode is RCTWM_WeightedArrive or RCTWM_WeightedBoth, the weight of the arriving tangent on the left side.
	* @param InNewValue	Specifies the new arriving tangent weight. Represents the length of the hypotenuse in the form of "sqrt(x*x+y*y)" using the same definitions for x and y as tangents.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetArriveTangentWeight(float InNewValue)
	{
		FMovieSceneDoubleValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.ArriveTangentWeight = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}

	/**
	* If Tangent Weight Mode is RCTWM_WeightedLeave or RCTWM_WeightedBoth, the weight of the leaving tangent on the right side.
	* @return Tangent Weight. Represents the length of the hypotenuse in the form of "sqrt(x*x+y*y)" using the same definitions for x and y as tangents.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	float GetLeaveTangentWeight() const
	{
		return GetValueFromChannel(KeyHandle).Tangent.LeaveTangentWeight;
	}

	/**
	* If Tangent Weight Mode is RCTWM_WeightedLeave or RCTWM_WeightedBoth, the weight of the leaving tangent on the right side.
	* @param InNewValue	Specifies the new leaving tangent weight. Represents the length of the hypotenuse in the form of "sqrt(x*x+y*y)" using the same definitions for x and y as tangents.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetLeaveTangentWeight(float InNewValue)
	{
		FMovieSceneDoubleValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.LeaveTangentWeight = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
};

UCLASS(BlueprintType)
class UMovieSceneScriptingDoubleChannel : public UMovieSceneScriptingChannel
{
	GENERATED_BODY()

	using Impl = TMovieSceneScriptingChannel<FMovieSceneDoubleChannel, UMovieSceneScriptingDoubleKey, double>;

public:
	/**
	* Add a key to this channel. This initializes a new key and returns a reference to it.
	* @param	InTime			The frame this key should go on. Respects TimeUnit to determine if it is a display rate frame or a tick resolution frame.
	* @param	NewValue		The value that this key should be created with.
	* @param	SubFrame		Optional [0-1) clamped sub-frame to put this key on. Ignored if TimeUnit is set to Tick Resolution.
	* @param	TimeUnit 		Is the specified InTime in Display Rate frames or Tick Resolution.
	* @param	InInterpolation	Interpolation method the key should use.
	* @return	The key that was created with the specified values at the specified time.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Add Key (Double)"))
	UMovieSceneScriptingDoubleKey* AddKey(const FFrameNumber& InTime, double NewValue, float SubFrame = 0.f, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, EMovieSceneKeyInterpolation InInterpolation = EMovieSceneKeyInterpolation::Auto)
	{
		return Impl::AddKeyInChannel(ChannelHandle, OwningSequence, OwningSection, InTime, NewValue, SubFrame, TimeUnit, InInterpolation);
	}

	/**
	* Removes the specified key. Does nothing if the key is not specified or the key belongs to another channel.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Remove Key (Double)"))
	virtual void RemoveKey(UMovieSceneScriptingKey* Key)
	{
		Impl::RemoveKeyFromChannel(ChannelHandle, Key);
	}

	/**
	* Gets all of the keys in this channel.
	* @return	An array of UMovieSceneScriptingDoubleKey's contained by this channel.
	*			Returns all keys even if clipped by the owning section's boundaries or outside of the current sequence play range.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Keys (Double)"))
	virtual TArray<UMovieSceneScriptingKey*> GetKeys() const override
	{
		return Impl::GetKeysInChannel(ChannelHandle, OwningSequence, OwningSection);
	}

	/**
	* Gets the keys in this channel specified by the specific index
	* @Indices  The indices from which to get the keys from
	* @return	An array of UMovieSceneScriptingKey's contained by this channel.
	*			Returns all keys specified by the indices, even if out of range.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Keys By Index (Double)"))
	virtual TArray<UMovieSceneScriptingKey*> GetKeysByIndex(const TArray<int32>& Indices) const override
	{
		return Impl::GetKeysInChannelByIndex(ChannelHandle, OwningSequence, OwningSection, Indices);
	}
	/**
	* Returns number of keys in this channel.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Num Keys (Double)"))
	int32 GetNumKeys() const
	{
		return ChannelHandle.Get() ? ChannelHandle.Get()->GetNumKeys() : 0;
	}

	/**
	* Gets baked keys in this channel.
	* @return	An array of double's contained by this channel.
	*			Returns baked keys in the specified range.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Evaluate Keys (Double)"))
	TArray<double> EvaluateKeys(FSequencerScriptingRange Range, FFrameRate FrameRate) const
	{
		return Impl::EvaluateKeysInChannel(ChannelHandle, OwningSequence, Range, FrameRate);
	}

	/**
	* Compute the effective range of this channel, for example, the extents of its key times
	*
	* @return A range that represents the greatest range of times occupied by this channel, in the sequence's frame resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Compute Effective Range (Double)"))
	FSequencerScriptingRange ComputeEffectiveRange() const
	{
		return Impl::ComputeEffectiveRangeInChannel(ChannelHandle, OwningSequence);
	}

	/**
	* @return Gets the Pre-infinity extrapolation state. See ERichCurveExtrapolation for more detail.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	ERichCurveExtrapolation GetPreInfinityExtrapolation() const
	{
		FMovieSceneDoubleChannel* Channel = ChannelHandle.Get();
		if (Channel)
		{
			return Channel->PreInfinityExtrap;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to get pre-infinity extrapolation."), ELogVerbosity::Error);
		return ERichCurveExtrapolation::RCCE_None;
	}

	/**
	* Sets the Pre-infinity extrapolation state. See ERichCurveExtrapolation for more detail.
	* @param InExtrapolation The new extrapolation mode this key should use for evaluating before this key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetPreInfinityExtrapolation(ERichCurveExtrapolation InExtrapolation)
	{
		FMovieSceneDoubleChannel* Channel = ChannelHandle.Get();
		if (Channel)
		{
			Channel->PreInfinityExtrap = InExtrapolation;
			return;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to set pre-infinity extrapolation."), ELogVerbosity::Error);
	}

	/**
	* @return Gets the Post-infinity extrapolation state. See ERichCurveExtrapolation for more detail.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	ERichCurveExtrapolation GetPostInfinityExtrapolation() const
	{
		FMovieSceneDoubleChannel* Channel = ChannelHandle.Get();
		if (Channel)
		{
			return Channel->PostInfinityExtrap;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to get post-infinity extrapolation."), ELogVerbosity::Error);
		return ERichCurveExtrapolation::RCCE_None;
	}

	/**
	* Sets the Post-infinity extrapolation state. See ERichCurveExtrapolation for more detail.
	* @param InExtrapolation The new extrapolation mode this key should use for evaluating after this key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetPostInfinityExtrapolation(ERichCurveExtrapolation InExtrapolation)
	{
		FMovieSceneDoubleChannel* Channel = ChannelHandle.Get();
		if (Channel)
		{
			Channel->PostInfinityExtrap = InExtrapolation;
			return;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to set post-infinity extrapolation."), ELogVerbosity::Error);
	}

	/**
	* Set this channel's default value that should be used when no keys are present.
	* Sets bHasDefaultValue to true automatically.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Default (Double)"))
	void SetDefault(double InDefaultValue)
	{
		Impl::SetDefaultInChannel(ChannelHandle, OwningSequence, OwningSection, InDefaultValue);
	}

	/**
	* Get this channel's default value that will be used when no keys are present. Only a valid
	* value when HasDefault() returns true.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Default (Double)"))
	double GetDefault() const
	{
		TOptional<double> DefaultValue = Impl::GetDefaultFromChannel(ChannelHandle);
		return DefaultValue.IsSet() ? DefaultValue.GetValue() : 0.f;
	}

	/**
	* Remove this channel's default value causing the channel to have no effect where no keys are present
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Remove Default (Double)"))
	void RemoveDefault()
	{
		Impl::RemoveDefaultFromChannel(ChannelHandle, OwningSequence, OwningSection);
	}

	/**
	* @return Does this channel have a default value set?
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Has Default (Double)"))
	bool HasDefault() const
	{
		return Impl::GetDefaultFromChannel(ChannelHandle).IsSet();
	}

	/**
	 * Transform the keys in time in the channel by an offset, scale and pivot
	 * @param	OffsetFrame	    The amount to offset the keys by
	 * @param	Scale	        The amount to scale the keys by
	 * @param	PivotFrame	    The frame to pivot around when scaling the keys
	 * @param   ScriptingRange  The time range of the keys to transform
	 * @param	TimeUnit 		Is the specified OffsetFrame, PivotFrame, and ScriptingRange in Display Rate frames or Tick Resolution.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Transform (Double)"))
	void Transform(FFrameNumber OffsetFrame, double Scale, FFrameNumber PivotFrame, FSequencerScriptingRange ScriptingRange, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate)
	{
		Impl::TransformKeysInChannel(ChannelHandle, OwningSequence, OwningSection, OffsetFrame, Scale, PivotFrame, ScriptingRange, TimeUnit);
	}

public:
	TWeakObjectPtr<UMovieSceneSequence> OwningSequence;
	TMovieSceneChannelHandle<FMovieSceneDoubleChannel> ChannelHandle;
	TWeakObjectPtr<UMovieSceneSection> OwningSection;
};

