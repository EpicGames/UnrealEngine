// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelData.h"
#include "MovieSceneScriptingChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "KeyParams.h"
#include "MovieSceneTimeUnit.h"

#include "MovieSceneScriptingFloat.generated.h"

class UMovieSceneScriptingChannel;
class UMovieSceneScriptingKey;
template <typename ChannelType, typename ScriptingKeyType, typename ScriptingKeyValueType> struct TMovieSceneScriptingChannel;

/**
* Exposes a Sequencer float type key to Python/Blueprints.
* Stores a reference to the data so changes to this class are forwarded onto the underlying data structures.
*/
UCLASS(abstract, BlueprintType)
class UMovieSceneScriptingFloatKey : public UMovieSceneScriptingKey
{
	GENERATED_BODY()
public:
	/**
	* Gets the time for this key from the owning channel.
	* @param TimeUnit	Should the time be returned in Display Rate frames (possibly with a sub-frame value) or in Tick Resolution with no sub-frame values?
	* @return			The time of this key which combines both the frame number and the sub-frame it is on. Sub-frame will be zero if you request Tick Resolution.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Time (Float)"))
	virtual FFrameTime GetTime(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) const override PURE_VIRTUAL(UMovieSceneScriptingFloatKey::GetTime, return FFrameTime(););
	
	/**
	* Sets the time for this key in the owning channel. Will replace any key that already exists at that frame number in this channel.
	* @param NewFrameNumber	What frame should this key be moved to? This should be in the time unit specified by TimeUnit.
	* @param SubFrame		If using Display Rate time, what is the sub-frame this should go to? Clamped [0-1), and ignored with when TimeUnit is set to Tick Resolution.
	* @param TimeUnit		Should the NewFrameNumber be interpreted as Display Rate frames or in Tick Resolution?
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Time (Float)"))
	virtual void SetTime(const FFrameNumber& NewFrameNumber, float SubFrame = 0.f, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) PURE_VIRTUAL(UMovieSceneScriptingFloatKey::SetTime);

	/**
	* Gets the value for this key from the owning channel.
	* @return	The float value this key represents.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Value (Float)"))
	virtual float GetValue() const PURE_VIRTUAL(UMovieSceneScriptingFloatKey::GetValue, return 0;);

	/**
	* Sets the value for this key, reflecting it in the owning channel.
	* @param InNewValue	The new float value for this key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Value (Float)"))
	virtual void SetValue(float InNewValue) PURE_VIRTUAL(UMovieSceneScriptingFloatKey::SetValue);

	/**
	* Gets the interpolation mode for this key from the owning channel.
	* @return	Interpolation mode this key uses to interpolate between this key and the next.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual ERichCurveInterpMode GetInterpolationMode() const PURE_VIRTUAL(UMovieSceneScriptingFloatKey::GetInterpolationMode, return ERichCurveInterpMode::RCIM_None;);

	/**
	* Sets the interpolation mode for this key, reflecting it in the owning channel.
	* @param InNewValue	Interpolation mode this key should use to interpolate between this key and the next.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual void SetInterpolationMode(ERichCurveInterpMode InNewValue) PURE_VIRTUAL(UMovieSceneScriptingFloatKey::SetInterpolationMode);

	/**
	* Gets the tangent mode for this key from the owning channel.
	* @return	Tangent mode that this key is using specifying which tangent values are respected when evaluating.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual ERichCurveTangentMode GetTangentMode() const PURE_VIRTUAL(UMovieSceneScriptingFloatKey::GetTangentMode, return ERichCurveTangentMode::RCTM_None;);

	/**
	* Sets the tangent mode for this key, reflecting it in the owning channel.
	* @param InNewValue	Tangent mode that this key should use to specify which tangent values are respected when evaluating. See ERichCurveTangentMode for more details.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual void SetTangentMode(ERichCurveTangentMode InNewValue) PURE_VIRTUAL(UMovieSceneScriptingFloatKey::SetTangentMode);

	/**
	* If Interpolation Mode is RCIM_Cubic, the arriving tangent at this key
	* @return Arrival Tangent value. Represents the geometric tangents in the form of "tan(y/x)" where y is the key's value and x is the seconds (both relative to key)
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual float GetArriveTangent() const PURE_VIRTUAL(UMovieSceneScriptingFloatKey::GetArriveTangent, return 0;);

	/**
	* If Interpolation Mode is RCIM_Cubic, the arriving tangent at this key.
	* @param InNewValue	Represents the geometric tangents in the form of "tan(y/x)" where y is the key's value and x is the seconds (both relative to key)
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual void SetArriveTangent(float InNewValue) PURE_VIRTUAL(UMovieSceneScriptingFloatKey::SetArriveTangent);

	/**
	* If Interpolation Mode is RCIM_Cubic, the leaving tangent at this key
	* @return Leaving Tangent value. Represents the geometric tangents in the form of "tan(y/x)" where y is the key's value and x is the seconds (both relative to key)
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual float GetLeaveTangent() const PURE_VIRTUAL(UMovieSceneScriptingFloatKey::GetLeaveTangent, return 0;);

	/**
	* If Interpolation Mode is RCIM_Cubic, the leaving tangent at this key.
	* @param InNewValue	Represents the geometric tangents in the form of "tan(y/x)" where y is the key's value and x is the seconds (both relative to key)
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual void SetLeaveTangent(float InNewValue) PURE_VIRTUAL(UMovieSceneScriptingFloatKey::SetLeaveTangent);

	/**
	* If Interpolation Mode is RCIM_Cubic, the tangent weight mode at this key
	* @return Tangent Weight Mode. See ERichCurveTangentWeightMode for more detail on what each mode does.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual ERichCurveTangentWeightMode GetTangentWeightMode() const PURE_VIRTUAL(UMovieSceneScriptingFloatKey::GetTangentWeightMode, return ERichCurveTangentWeightMode::RCTWM_WeightedNone;);

	/**
	* If Interpolation Mode is RCIM_Cubic, the tangent weight mode at this key.
	* @param InNewValue	Specifies which tangent weights should be respected when evaluating the key. 
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual void SetTangentWeightMode(ERichCurveTangentWeightMode InNewValue) PURE_VIRTUAL(UMovieSceneScriptingFloatKey::SetTangentWeightMode);

	/**
	* If Tangent Weight Mode is RCTWM_WeightedArrive or RCTWM_WeightedBoth, the weight of the arriving tangent on the left side.
	* @return Tangent Weight. Represents the length of the hypotenuse in the form of "sqrt(x*x+y*y)" using the same definitions for x and y as tangents.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual float GetArriveTangentWeight() const PURE_VIRTUAL(UMovieSceneScriptingFloatKey::GetArriveTangentWeight, return 0;);

	/**
	* If Tangent Weight Mode is RCTWM_WeightedArrive or RCTWM_WeightedBoth, the weight of the arriving tangent on the left side.
	* @param InNewValue	Specifies the new arriving tangent weight. Represents the length of the hypotenuse in the form of "sqrt(x*x+y*y)" using the same definitions for x and y as tangents.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual void SetArriveTangentWeight(float InNewValue) PURE_VIRTUAL(UMovieSceneScriptingFloatKey::SetArriveTangentWeight);

	/**
	* If Tangent Weight Mode is RCTWM_WeightedLeave or RCTWM_WeightedBoth, the weight of the leaving tangent on the right side.
	* @return Tangent Weight. Represents the length of the hypotenuse in the form of "sqrt(x*x+y*y)" using the same definitions for x and y as tangents.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual float GetLeaveTangentWeight() const PURE_VIRTUAL(UMovieSceneScriptingFloatKey::GetLeaveTangentWeight, return 0;);

	/**
	* If Tangent Weight Mode is RCTWM_WeightedLeave or RCTWM_WeightedBoth, the weight of the leaving tangent on the right side.
	* @param InNewValue	Specifies the new leaving tangent weight. Represents the length of the hypotenuse in the form of "sqrt(x*x+y*y)" using the same definitions for x and y as tangents.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	virtual void SetLeaveTangentWeight(float InNewValue) PURE_VIRTUAL(UMovieSceneScriptingFloatKey::SetLeaveTangentWeight);
};

UCLASS(BlueprintType)
class UMovieSceneScriptingActualFloatKey : public UMovieSceneScriptingFloatKey, public TMovieSceneScriptingKey<FMovieSceneFloatChannel, FMovieSceneFloatValue>
{
	GENERATED_BODY()
public:
	virtual FFrameTime GetTime(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) const override
	{
		return GetTimeFromChannel(KeyHandle, OwningSequence, TimeUnit);
	}
	virtual void SetTime(const FFrameNumber& NewFrameNumber, float SubFrame = 0.f, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) override
	{
		SetTimeInChannel(KeyHandle, OwningSequence, OwningSection, NewFrameNumber, TimeUnit, SubFrame);
	}
	virtual float GetValue() const override
	{
		return GetValueFromChannel(KeyHandle).Value;
	}
	virtual void SetValue(float InNewValue) override
	{
		FMovieSceneFloatValue ExistValue = GetValueFromChannel(KeyHandle);
		ExistValue.Value = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistValue);
	}
	virtual ERichCurveInterpMode GetInterpolationMode() const override
	{
		return GetValueFromChannel(KeyHandle).InterpMode;
	}
	virtual void SetInterpolationMode(ERichCurveInterpMode InNewValue) override
	{
		FMovieSceneFloatValue ExistValue = GetValueFromChannel(KeyHandle);
		ExistValue.InterpMode = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistValue);
	}
	virtual ERichCurveTangentMode GetTangentMode() const override
	{
		return GetValueFromChannel(KeyHandle).TangentMode;
	}
	virtual void SetTangentMode(ERichCurveTangentMode InNewValue) override
	{
		FMovieSceneFloatValue ExistValue = GetValueFromChannel(KeyHandle);
		ExistValue.TangentMode = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistValue);
	}
	virtual float GetArriveTangent() const override
	{
		return GetValueFromChannel(KeyHandle).Tangent.ArriveTangent;
	}
	virtual void SetArriveTangent(float InNewValue) override
	{
		FMovieSceneFloatValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.ArriveTangent = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
	virtual float GetLeaveTangent() const override
	{
		return GetValueFromChannel(KeyHandle).Tangent.LeaveTangent;
	}
	virtual void SetLeaveTangent(float InNewValue) override
	{
		FMovieSceneFloatValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.LeaveTangent = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
	virtual ERichCurveTangentWeightMode GetTangentWeightMode() const override
	{
		return GetValueFromChannel(KeyHandle).Tangent.TangentWeightMode;
	}
	virtual void SetTangentWeightMode(ERichCurveTangentWeightMode InNewValue) override
	{
		FMovieSceneFloatValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.TangentWeightMode = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
	virtual float GetArriveTangentWeight() const override
	{
		return GetValueFromChannel(KeyHandle).Tangent.ArriveTangentWeight;
	}
	virtual void SetArriveTangentWeight(float InNewValue) override
	{
		FMovieSceneFloatValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.ArriveTangentWeight = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
	virtual float GetLeaveTangentWeight() const override
	{
		return GetValueFromChannel(KeyHandle).Tangent.LeaveTangentWeight;
	}
	virtual void SetLeaveTangentWeight(float InNewValue) override
	{
		FMovieSceneFloatValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.LeaveTangentWeight = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
};

UCLASS(BlueprintType)
class UMovieSceneScriptingDoubleAsFloatKey : public UMovieSceneScriptingFloatKey, public TMovieSceneScriptingKey<FMovieSceneDoubleChannel, FMovieSceneDoubleValue>
{
	GENERATED_BODY()
public:
	virtual FFrameTime GetTime(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) const override
	{
		return GetTimeFromChannel(KeyHandle, OwningSequence, TimeUnit);
	}
	virtual void SetTime(const FFrameNumber& NewFrameNumber, float SubFrame = 0.f, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) override
	{
		SetTimeInChannel(KeyHandle, OwningSequence, OwningSection, NewFrameNumber, TimeUnit, SubFrame);
	}
	virtual float GetValue() const override
	{
		return (float)GetValueFromChannel(KeyHandle).Value;
	}
	virtual void SetValue(float InNewValue) override
	{
		FMovieSceneDoubleValue ExistValue = GetValueFromChannel(KeyHandle);
		ExistValue.Value = (double)InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistValue);
	}
	virtual ERichCurveInterpMode GetInterpolationMode() const override
	{
		return GetValueFromChannel(KeyHandle).InterpMode;
	}
	virtual void SetInterpolationMode(ERichCurveInterpMode InNewValue) override
	{
		FMovieSceneDoubleValue ExistValue = GetValueFromChannel(KeyHandle);
		ExistValue.InterpMode = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistValue);
	}
	virtual ERichCurveTangentMode GetTangentMode() const override
	{
		return GetValueFromChannel(KeyHandle).TangentMode;
	}
	virtual void SetTangentMode(ERichCurveTangentMode InNewValue) override
	{
		FMovieSceneDoubleValue ExistValue = GetValueFromChannel(KeyHandle);
		ExistValue.TangentMode = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistValue);
	}
	virtual float GetArriveTangent() const override
	{
		return GetValueFromChannel(KeyHandle).Tangent.ArriveTangent;
	}
	virtual void SetArriveTangent(float InNewValue) override
	{
		FMovieSceneDoubleValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.ArriveTangent = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
	virtual float GetLeaveTangent() const override
	{
		return GetValueFromChannel(KeyHandle).Tangent.LeaveTangent;
	}
	virtual void SetLeaveTangent(float InNewValue) override
	{
		FMovieSceneDoubleValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.LeaveTangent = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
	virtual ERichCurveTangentWeightMode GetTangentWeightMode() const override
	{
		return GetValueFromChannel(KeyHandle).Tangent.TangentWeightMode;
	}
	virtual void SetTangentWeightMode(ERichCurveTangentWeightMode InNewValue) override
	{
		FMovieSceneDoubleValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.TangentWeightMode = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
	virtual float GetArriveTangentWeight() const override
	{
		return GetValueFromChannel(KeyHandle).Tangent.ArriveTangentWeight;
	}
	virtual void SetArriveTangentWeight(float InNewValue) override
	{
		FMovieSceneDoubleValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.ArriveTangentWeight = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
	virtual float GetLeaveTangentWeight() const override
	{
		return GetValueFromChannel(KeyHandle).Tangent.LeaveTangentWeight;
	}
	virtual void SetLeaveTangentWeight(float InNewValue) override
	{
		FMovieSceneDoubleValue ExistKeyValue = GetValueFromChannel(KeyHandle);
		FMovieSceneTangentData& ExistTangentData = ExistKeyValue.Tangent;
		ExistTangentData.LeaveTangentWeight = InNewValue;
		SetValueInChannel(KeyHandle, OwningSection, ExistKeyValue);
	}
};

UCLASS(BlueprintType)
class UMovieSceneScriptingFloatChannel : public UMovieSceneScriptingChannel
{
	GENERATED_BODY()

	using FloatImpl = TMovieSceneScriptingChannel<FMovieSceneFloatChannel, UMovieSceneScriptingActualFloatKey, float>;
	using DoubleImpl = TMovieSceneScriptingChannel<FMovieSceneDoubleChannel, UMovieSceneScriptingDoubleAsFloatKey, double>;

public:
	/**
	* Add a key to this channel. This initializes a new key and returns a reference to it.
	* @param	InTime			The frame this key should go on. Respects TimeUnit to determine if it is a display rate frame or a tick resolution frame.
	* @param	NewValue		The value that this key should be created with.
	* @param	SubFrame		Optional [0-1) clamped sub-frame to put this key on. Ignored if TimeUnit is set to Tick Resolution.
	* @param	TimeUnit 		Is the specified InTime in Display Rate frames or Tick Resolution.
	* @param	InInterpolation	Interpolation method the key should use.
	* @return	The key that was created with the specified values at the specified time.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Add Key (Float)"))
	UMovieSceneScriptingFloatKey* AddKey(const FFrameNumber& InTime, float NewValue, float SubFrame = 0.f, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate, EMovieSceneKeyInterpolation InInterpolation = EMovieSceneKeyInterpolation::Auto)
	{
		if (FloatChannelHandle.Get())
		{
			return FloatImpl::AddKeyInChannel(FloatChannelHandle, OwningSequence, OwningSection, InTime, NewValue, SubFrame, TimeUnit, InInterpolation);
		}
		else
		{
			double DoubleNewValue(NewValue);
			return DoubleImpl::AddKeyInChannel(DoubleChannelHandle, OwningSequence, OwningSection, InTime, DoubleNewValue, SubFrame, TimeUnit, InInterpolation);
		}
	}

	/**
	* Removes the specified key. Does nothing if the key is not specified or the key belongs to another channel.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Remove Key (Float)"))
	virtual void RemoveKey(UMovieSceneScriptingKey* Key)
	{
		if (FloatChannelHandle.Get())
		{
			FloatImpl::RemoveKeyFromChannel(FloatChannelHandle, Key);
		}
		else
		{
			DoubleImpl::RemoveKeyFromChannel(DoubleChannelHandle, Key);
		}
	}

	/**
	* Gets all of the keys in this channel.
	* @return	An array of UMovieSceneScriptingFloatKey's contained by this channel.
	*			Returns all keys even if clipped by the owning section's boundaries or outside of the current sequence play range.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Keys (Float)"))
	virtual TArray<UMovieSceneScriptingKey*> GetKeys() const override
	{
		if (FloatChannelHandle.Get())
		{
			return FloatImpl::GetKeysInChannel(FloatChannelHandle, OwningSequence, OwningSection);
		}
		else
		{
			return DoubleImpl::GetKeysInChannel(DoubleChannelHandle, OwningSequence, OwningSection);
		}
	}

	/**
	* Gets the keys in this channel specified by the specific index
	* @Indices  The indices from which to get the keys from
	* @return	An array of UMovieSceneScriptingKey's contained by this channel.
	*			Returns all keys specified by the indices, even if out of range.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Keys By Index (Float)"))
	virtual TArray<UMovieSceneScriptingKey*> GetKeysByIndex(const TArray<int32>& Indices) const override
	{
		if (FloatChannelHandle.Get())
		{
			return FloatImpl::GetKeysInChannelByIndex(FloatChannelHandle, OwningSequence, OwningSection, Indices);
		}
		else
		{
			return DoubleImpl::GetKeysInChannelByIndex(DoubleChannelHandle, OwningSequence, OwningSection, Indices);
		}
	}

	/**
	* Returns number of keys in this channel.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Num Keys (Float)"))
	int32 GetNumKeys() const
	{
		if (FloatChannelHandle.Get())
		{
			return FloatChannelHandle.Get()->GetNumKeys();
		}
		else if (DoubleChannelHandle.Get())
		{
			return DoubleChannelHandle.Get()->GetNumKeys();
		}
		else
		{
			return 0;
		}
	}

	/**
	* Gets baked keys in this channel.
	* @return	An array of float's contained by this channel.
	*			Returns baked keys in the specified range.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Evaluate Keys (Float)"))
	TArray<float> EvaluateKeys(FSequencerScriptingRange Range, FFrameRate FrameRate) const
	{
		if (FloatChannelHandle.Get())
		{
			return FloatImpl::EvaluateKeysInChannel(FloatChannelHandle, OwningSequence, Range, FrameRate);
		}
		else
		{
			TArray<double> DoubleValues = DoubleImpl::EvaluateKeysInChannel(DoubleChannelHandle, OwningSequence, Range, FrameRate);

			TArray<float> FloatValues;
			Algo::Transform(DoubleValues, FloatValues, [](double Value) { return (float)Value; });
			return FloatValues;
		}
	}

	/**
	* Compute the effective range of this channel, for example, the extents of its key times
	*
	* @return A range that represents the greatest range of times occupied by this channel, in the sequence's frame resolution
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Compute Effective Range (Float)"))
	FSequencerScriptingRange ComputeEffectiveRange() const
	{
		if (FloatChannelHandle.Get())
		{
			return FloatImpl::ComputeEffectiveRangeInChannel(FloatChannelHandle, OwningSequence);
		}
		else
		{
			return DoubleImpl::ComputeEffectiveRangeInChannel(DoubleChannelHandle, OwningSequence);
		}
	}

	/**
	* @return Gets the Pre-infinity extrapolation state. See ERichCurveExtrapolation for more detail.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	ERichCurveExtrapolation GetPreInfinityExtrapolation() const
	{
		FMovieSceneFloatChannel* FloatChannel = FloatChannelHandle.Get();
		if (FloatChannel)
		{
			return FloatChannel->PreInfinityExtrap;
		}

		FMovieSceneDoubleChannel* DoubleChannel = DoubleChannelHandle.Get();
		if (DoubleChannel)
		{
			return DoubleChannel->PreInfinityExtrap;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to get pre-infinity extrapolation."), ELogVerbosity::Error);
		return ERichCurveExtrapolation::RCCE_None;
	}

	/**
	* Sets the Pre-infinity extrapolation state. See ERichCurveExtrapolation for more detail.
	* @param InExtrapolation The new extrapolation mode this key should use for evaluating before this key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetPreInfinityExtrapolation(ERichCurveExtrapolation InExtrapolation)
	{
		FMovieSceneFloatChannel* FloatChannel = FloatChannelHandle.Get();
		if (FloatChannel)
		{
			FloatChannel->PreInfinityExtrap = InExtrapolation;
			return;
		}

		FMovieSceneDoubleChannel* DoubleChannel = DoubleChannelHandle.Get();
		if (DoubleChannel)
		{
			DoubleChannel->PreInfinityExtrap = InExtrapolation;
			return;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to set pre-infinity extrapolation."), ELogVerbosity::Error);
	}

	/**
	* @return Gets the Post-infinity extrapolation state. See ERichCurveExtrapolation for more detail.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	ERichCurveExtrapolation GetPostInfinityExtrapolation() const
	{
		FMovieSceneFloatChannel* FloatChannel = FloatChannelHandle.Get();
		if (FloatChannel)
		{
			return FloatChannel->PostInfinityExtrap;
		}

		FMovieSceneDoubleChannel* DoubleChannel = DoubleChannelHandle.Get();
		if (DoubleChannel)
		{
			return DoubleChannel->PostInfinityExtrap;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to get post-infinity extrapolation."), ELogVerbosity::Error);
		return ERichCurveExtrapolation::RCCE_None;
	}

	/**
	* Sets the Post-infinity extrapolation state. See ERichCurveExtrapolation for more detail.
	* @param InExtrapolation The new extrapolation mode this key should use for evaluating after this key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys")
	void SetPostInfinityExtrapolation(ERichCurveExtrapolation InExtrapolation)
	{
		FMovieSceneFloatChannel* FloatChannel = FloatChannelHandle.Get();
		if (FloatChannel)
		{
			FloatChannel->PostInfinityExtrap = InExtrapolation;
			return;
		}

		FMovieSceneDoubleChannel* DoubleChannel = DoubleChannelHandle.Get();
		if (DoubleChannel)
		{
			DoubleChannel->PostInfinityExtrap = InExtrapolation;
			return;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to set post-infinity extrapolation."), ELogVerbosity::Error);
	}

	/**
	* Set this channel's default value that should be used when no keys are present.
	* Sets bHasDefaultValue to true automatically.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Default (Float)"))
	void SetDefault(float InDefaultValue)
	{
		if (FloatChannelHandle.Get())
		{
			FloatImpl::SetDefaultInChannel(FloatChannelHandle, OwningSequence, OwningSection, InDefaultValue);
		}
		else
		{
			double DoubleDefaultValue(InDefaultValue);
			DoubleImpl::SetDefaultInChannel(DoubleChannelHandle, OwningSequence, OwningSection, DoubleDefaultValue);
		}
	}

	/**
	* Get this channel's default value that will be used when no keys are present. Only a valid
	* value when HasDefault() returns true.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Default (Float)"))
	float GetDefault() const
	{
		if (FloatChannelHandle.Get())
		{
			TOptional<float> DefaultValue = FloatImpl::GetDefaultFromChannel(FloatChannelHandle);
			return DefaultValue.IsSet() ? DefaultValue.GetValue() : 0.f;
		}
		else
		{
			TOptional<double> DefaultValue = DoubleImpl::GetDefaultFromChannel(DoubleChannelHandle);
			return DefaultValue.IsSet() ? (float)DefaultValue.GetValue() : 0.f;
		}
	}

	/**
	* Remove this channel's default value causing the channel to have no effect where no keys are present
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Remove Default (Float)"))
	void RemoveDefault()
	{
		if (FloatChannelHandle.Get())
		{
			FloatImpl::RemoveDefaultFromChannel(FloatChannelHandle, OwningSequence, OwningSection);
		}
		else
		{
			DoubleImpl::RemoveDefaultFromChannel(DoubleChannelHandle, OwningSequence, OwningSection);
		}
	}

	/**
	* @return Does this channel have a default value set?
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Has Default (Float)"))
	bool HasDefault() const
	{
		if (FloatChannelHandle.Get())
		{
			return FloatImpl::GetDefaultFromChannel(FloatChannelHandle).IsSet();
		}
		else
		{
			return DoubleImpl::GetDefaultFromChannel(DoubleChannelHandle).IsSet();
		}
	}

	/**
	 * Transform the keys in time in the channel by an offset, scale and pivot
	 * @param	OffsetFrame	    The amount to offset the keys by
	 * @param	Scale	        The amount to scale the keys by
	 * @param	PivotFrame	    The frame to pivot around when scaling the keys
	 * @param   ScriptingRange  The time range of the keys to transform
	 * @param	TimeUnit 		Is the specified OffsetFrame, PivotFrame, and ScriptingRange in Display Rate frames or Tick Resolution.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Transform (Float)"))
	void Transform(FFrameNumber OffsetFrame, double Scale, FFrameNumber PivotFrame, FSequencerScriptingRange ScriptingRange, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate)
	{
		if (FloatChannelHandle.Get())
		{
			FloatImpl::TransformKeysInChannel(FloatChannelHandle, OwningSequence, OwningSection, OffsetFrame, Scale, PivotFrame, ScriptingRange, TimeUnit);
		}
		else
		{
			DoubleImpl::TransformKeysInChannel(DoubleChannelHandle, OwningSequence, OwningSection, OffsetFrame, Scale, PivotFrame, ScriptingRange, TimeUnit);
		}
	}

public:
	TWeakObjectPtr<UMovieSceneSequence> OwningSequence;
	TWeakObjectPtr<UMovieSceneSection> OwningSection;
private:
	template<typename ChannelType, typename ScriptingChannelType>
	friend void SetScriptingChannelHandle(ScriptingChannelType* ScriptingChannel, FMovieSceneChannelProxy& ChannelProxy, int32 ChannelIndex);
	TMovieSceneChannelHandle<FMovieSceneFloatChannel> FloatChannelHandle;
	TMovieSceneChannelHandle<FMovieSceneDoubleChannel> DoubleChannelHandle;
};
