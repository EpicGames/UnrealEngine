// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneScriptingChannel.h"
#include "Channels/MovieSceneTextChannel.h"
#include "KeyParams.h"
#include "MovieSceneTimeUnit.h"
#include "MovieSceneScriptingText.generated.h"

class UMovieSceneScriptingChannel;
class UMovieSceneScriptingKey;
template <typename ChannelType, typename ScriptingKeyType, typename ScriptingKeyValueType> struct TMovieSceneScriptingChannel;

/**
 * Exposes a Sequencer text type key to Python/Blueprints.
 * Stores a reference to the data so changes to this class are forwarded onto the underlying data structures.
 */
UCLASS(BlueprintType)
class UMovieSceneScriptingTextKey : public UMovieSceneScriptingKey, public TMovieSceneScriptingKey<FMovieSceneTextChannel, FText>
{
	GENERATED_BODY()

public:
	/**
	* Gets the time for this key from the owning channel.
	* @param TimeUnit	Should the time be returned in Display Rate frames (possibly with a sub-frame value) or in Tick Resolution with no sub-frame values?
	* @return			The time of this key which combines both the frame number and the sub-frame it is on. Sub-frame will be zero if you request Tick Resolution.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Time (Text)"))
	virtual FFrameTime GetTime(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) const override
	{
		return GetTimeFromChannel(KeyHandle, OwningSequence, TimeUnit);
	}

	/**
	* Sets the time for this key in the owning channel. Will replace any key that already exists at that frame number in this channel.
	* @param NewFrameNumber	What frame should this key be moved to? This should be in the time unit specified by TimeUnit.
	* @param SubFrame		If using Display Rate time, what is the sub-frame this should go to? Clamped [0-1), and ignored with when TimeUnit is set to Tick Resolution.
	* @param TimeUnit		Should the NewFrameNumber be interpreted as Display Rate frames or in Tick Resolution?
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Time (Text)"))
	void SetTime(const FFrameNumber& NewFrameNumber, float SubFrame = 0.f, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate)
	{
		SetTimeInChannel(KeyHandle, OwningSequence, OwningSection, NewFrameNumber, TimeUnit, SubFrame);
	}

	/**
	* Gets the value for this key from the owning channel.
	* @return	The value for this key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Value (Text)"))
	FText GetValue() const
	{
		return GetValueFromChannel(KeyHandle);
	}

	/**
	* Sets the value for this key, reflecting it in the owning channel.
	* @param InNewValue	The new value for this key.
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Value (Text)"))
	void SetValue(const FText& InNewValue)
	{
		SetValueInChannel(KeyHandle, OwningSection, InNewValue);
	}
};

UCLASS(BlueprintType)
class UMovieSceneScriptingTextChannel : public UMovieSceneScriptingChannel
{
	GENERATED_BODY()

	using Impl = TMovieSceneScriptingChannel<FMovieSceneTextChannel, UMovieSceneScriptingTextKey, FText>;

public:
	/**
	 * Add a key to this channel. This initializes a new key and returns a reference to it.
	 * @param	InTime			The frame this key should go on. Respects TimeUnit to determine if it is a display rate frame or a tick resolution frame.
	 * @param	NewValue		The value that this key should be created with.
	 * @param	SubFrame		Optional [0-1) clamped sub-frame to put this key on. Ignored if TimeUnit is set to Tick Resolution.
	 * @param	TimeUnit 		Is the specified InTime in Display Rate frames or Tick Resolution.
	 * @return	The key that was created with the specified values at the specified time.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Add Key (Text)"))
	UMovieSceneScriptingTextKey* AddKey(const FFrameNumber& InTime, FText NewValue, float SubFrame = 0.f, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate)
	{
		return Impl::AddKeyInChannel(ChannelHandle, OwningSequence, OwningSection,InTime, NewValue, SubFrame, TimeUnit, EMovieSceneKeyInterpolation::Auto);
	}

	/**
	 * Removes the specified key. Does nothing if the key is not specified or the key belongs to another channel.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Remove Key (Text)"))
	virtual void RemoveKey(UMovieSceneScriptingKey* Key)
	{
		Impl::RemoveKeyFromChannel(ChannelHandle, Key);
	}

	/**
	 * Gets all of the keys in this channel.
	 * @return	An array of UMovieSceneScriptingTextKey's contained by this channel.
	 *			Returns all keys even if clipped by the owning section's boundaries or outside of the current sequence play range.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Keys (Text)"))
	virtual TArray<UMovieSceneScriptingKey*> GetKeys() const override
	{
		return Impl::GetKeysInChannel(ChannelHandle, OwningSequence, OwningSection);
	}

	/**
	 * Gets the keys in this channel specified by the specific index
	 * @Indices  The indices from which to get the keys from
	 * @return	An array of UMovieSceneScriptingKey's contained by this channel.
	 *			Returns all keys specified by the indices, even if out of range.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Keys By Index (Text)"))
	virtual TArray<UMovieSceneScriptingKey*> GetKeysByIndex(const TArray<int32>& Indices) const override
	{
		return Impl::GetKeysInChannelByIndex(ChannelHandle, OwningSequence, OwningSection, Indices);
	}

	/**
	 * Set this channel's default value that should be used when no keys are present.
	 * Sets bHasDefaultValue to true automatically.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Set Default (Text)"))
	void SetDefault(FText InDefaultValue)
	{
		Impl::SetDefaultInChannel(ChannelHandle, OwningSequence, OwningSection, InDefaultValue);
	}

	/**
	 * Get this channel's default value that will be used when no keys are present. Only a valid
	 * value when HasDefault() returns true.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Get Default (Text)"))
	FText GetDefault() const
	{
		TOptional<FText> DefaultValue = Impl::GetDefaultFromChannel(ChannelHandle);
		return DefaultValue.IsSet() ? DefaultValue.GetValue() : FText();
	}

	/**
	 * Remove this channel's default value causing the channel to have no effect where no keys are present
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Remove Default (Text)"))
	void RemoveDefault()
	{
		Impl::RemoveDefaultFromChannel(ChannelHandle, OwningSequence, OwningSection);
	}

	/**
	 * @return Does this channel have a default value set?
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Has Default (Text)"))
	bool HasDefault() const
	{
		return Impl::GetDefaultFromChannel(ChannelHandle).IsSet();
	}

	/**
	 * Transform the OffsetFrame in time in the channel by an offset, scale and pivot
	 * @param	OffsetFrame	    The amount to offset the keys by
	 * @param	Scale	        The amount to scale the keys by
	 * @param	PivotFrame	    The frame to pivot around when scaling the keys
	 * @param   ScriptingRange  The time range of the keys to transform
	 * @param	TimeUnit 		Is the specified OffsetFrame, PivotFrame, and ScriptingRange in Display Rate frames or Tick Resolution.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Keys", meta = (DisplayName = "Transform (Text)"))
	void Transform(FFrameNumber OffsetFrame, double Scale, FFrameNumber PivotFrame, FSequencerScriptingRange ScriptingRange, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate)
	{
		Impl::TransformKeysInChannel(ChannelHandle, OwningSequence, OwningSection, OffsetFrame, Scale, PivotFrame, ScriptingRange, TimeUnit);
	}

public:
	TWeakObjectPtr<UMovieSceneSequence> OwningSequence;
	TMovieSceneChannelHandle<FMovieSceneTextChannel> ChannelHandle;
	TWeakObjectPtr<UMovieSceneSection> OwningSection;
};
