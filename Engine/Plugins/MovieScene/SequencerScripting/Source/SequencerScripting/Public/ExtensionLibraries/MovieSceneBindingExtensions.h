// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneBindingExtensions.generated.h"

class UMovieSceneTrack;
struct FMovieSceneBindingProxy;
template <typename T> class TSubclassOf;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto FMovieSceneBindingProxies for scripting
 */
UCLASS()
class UMovieSceneBindingExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/**
	 * Check whether the specified binding is valid
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API bool IsValid(const FMovieSceneBindingProxy& InBinding);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Equal (MovieSceneBindingProxy)", CompactNodeTitle = "==", ScriptOperator = "==", BlueprintThreadSafe), Category = "Sequencer|Sequence")
	static UE_API bool EqualEqual_MovieSceneBindingProxy(FMovieSceneBindingProxy A, FMovieSceneBindingProxy B);

	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "Not Equal (MovieSceneBindingProxy)", CompactNodeTitle = "!=", ScriptOperator = "!=", BlueprintThreadSafe), Category = "Sequencer|Sequence")
	static UE_API bool NotEqual_MovieSceneBindingProxy(FMovieSceneBindingProxy A, FMovieSceneBindingProxy B);

	/**
	 * Get this binding's ID
	 *
	 * @param InBinding     The binding to get the ID of
	 * @return The guid that uniquely represents this binding
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FGuid GetId(const FMovieSceneBindingProxy& InBinding);

	/**
	 * Get this binding's name
	 *
	 * @param InBinding     The binding to get the name of
	 * @return The display name of the binding
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FText GetDisplayName(const FMovieSceneBindingProxy& InBinding);

	/**
	 * Set this binding's name
	 *
	 * @param InBinding     The binding to get the name of
	 * @param InName The display name of the binding
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API void SetDisplayName(const FMovieSceneBindingProxy& InBinding, const FText& InDisplayName);

	/**
	 * Get this binding's object non-display name
	 *
	 * @param InBinding     The binding to get the name of
	 * @return The name of the binding
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FString GetName(const FMovieSceneBindingProxy& InBinding);

	/**
	 * Set this binding's object non-display name
	 *
	 * @param InBinding     The binding to get the name of
	 * @param InName The name of the binding
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void SetName(const FMovieSceneBindingProxy& InBinding, const FString& InName);

	/**
	 * Get the sorting order for this binding
	 *
	 * @param InBinding        The binding to get the sorting order from
	 * @return The sorting order of the requested binding
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API int32 GetSortingOrder(const FMovieSceneBindingProxy& InBinding);

	/**
	 * Set the sorting order for this binding
	 *
	 * @param InBinding        The binding to get the sorting order from
	 * @param SortingOrder The sorting order to set
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API void SetSortingOrder(const FMovieSceneBindingProxy& InBinding, int32 SortingOrder);

	/**
	 * Get all the tracks stored within this binding
	 *
	 * @param InBinding     The binding to find tracks in
	 * @return An array containing all the binding's tracks
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API TArray<UMovieSceneTrack*> GetTracks(const FMovieSceneBindingProxy& InBinding);

	/**
	 * Find all tracks within a given binding of the specified type
	 *
	 * @param InBinding     The binding to find tracks in
	 * @param TrackType     A UMovieSceneTrack class type specifying which types of track to return
	 * @return An array containing any tracks that match the type specified
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod, DeterminesOutputType="TrackType"))
	static UE_API TArray<UMovieSceneTrack*> FindTracksByType(const FMovieSceneBindingProxy& InBinding, TSubclassOf<UMovieSceneTrack> TrackType);

	/**
	 * Find all tracks within a given binding of the specified type, not allowing sub-classed types
	 *
	 * @param InBinding     The binding to find tracks in
	 * @param TrackType     A UMovieSceneTrack class type specifying the exact types of track to return
	 * @return An array containing any tracks that are exactly the same as the type specified
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod, DeterminesOutputType="TrackType"))
	static UE_API TArray<UMovieSceneTrack*> FindTracksByExactType(const FMovieSceneBindingProxy& InBinding, TSubclassOf<UMovieSceneTrack> TrackType);

	/**
	 * Remove the specified track from this binding
	 *
	 * @param InBinding     The binding to remove the track from
	 * @param TrackToRemove The track to remove
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void RemoveTrack(const FMovieSceneBindingProxy& InBinding, UMovieSceneTrack* TrackToRemove);

	/**
	 * Remove the specified binding
	 *
	 * @param InBinding     The binding to remove the track from
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void Remove(const FMovieSceneBindingProxy& InBinding);

	/**
	 * Add a new track to the specified binding
	 *
	 * @param InBinding     The binding to add tracks to
	 * @param TrackType     A UMovieSceneTrack class type specifying the type of track to create
	 * @return The newly created track, if successful
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API UMovieSceneTrack* AddTrack(const FMovieSceneBindingProxy& InBinding, TSubclassOf<UMovieSceneTrack> TrackType);

	/**
	* Get all the children of this binding
	*
	* @param InBinding     The binding to to get children of
	* @return An array containing all the binding's children
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API TArray<FMovieSceneBindingProxy> GetChildPossessables(const FMovieSceneBindingProxy& InBinding);

	/**
	* Get this binding's object template
	*
	* @param InBinding     The binding to get the object template of
	* @return The object template of the binding
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API UObject* GetObjectTemplate(const FMovieSceneBindingProxy& InBinding);

	/**
	* Get this binding's possessed object class
	*
	* @param InBinding     The binding to get the possessed object class of
	* @return The possessed object class of the binding
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API UClass* GetPossessedObjectClass(const FMovieSceneBindingProxy& InBinding);

	/**
	* Get the parent of this binding
	*
	* @param InBinding     The binding to get the parent of
	* @return The binding's parent
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FMovieSceneBindingProxy GetParent(const FMovieSceneBindingProxy& InBinding);

	/**
	 * Set the parent to this binding
	 *
	 * @param InBinding     The binding to set 
	 * @param InParentBinding     The parent to set the InBinding to
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetParent(const FMovieSceneBindingProxy& InBinding, const FMovieSceneBindingProxy& InParentBinding);

	/**
     * Move all the contents (tracks, child bindings) of the specified binding ID onto another
	 *
	 * @param SourceBindingId The identifier of the binding ID to move all tracks and children from
	 * @param DestinationBindingId The identifier of the binding ID to move the contents to	 
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void MoveBindingContents(const FMovieSceneBindingProxy& SourceBindingId, const FMovieSceneBindingProxy& DestinationBindingId);

	/**
	 * Set the spawnable id that the possessable binding should possess
	 *
	 * @param InBinding     The binding to set
	 * @param SpawnableBindingID The spawnable's binding id
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetSpawnableBindingID(const FMovieSceneBindingProxy& InBinding, const FMovieSceneObjectBindingID& SpawnableBindingID);
};

#undef UE_API
