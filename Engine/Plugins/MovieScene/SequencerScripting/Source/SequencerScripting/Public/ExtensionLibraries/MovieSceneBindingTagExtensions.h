// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneBindingTagExtensions.generated.h"

struct FMovieSceneBindingProxy;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Extension methods for reading and writing Sequencer binding tags.
 *
 * Binding tags are name-based labels attached to object bindings in a
 * Sequence. They are authored interactively via RMB -> Expose on a binding
 * in the Sequencer editor, and queried at runtime via
 * UMovieSceneSequence::FindBindingByTag. These extensions expose the
 * authoring operations that are otherwise only available through the
 * editor UI.
 */
UCLASS()
class UMovieSceneBindingTagExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/** Get every tag name registered on the sequence's MovieScene. */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Tags", meta=(ScriptMethod))
	static UE_API TArray<FName> GetAllBindingTags(UMovieSceneSequence* Sequence);

	/** Get every tag currently attached to this specific binding. */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Tags", meta=(ScriptMethod))
	static UE_API TArray<FName> GetBindingTags(const FMovieSceneBindingProxy& Binding);

	/**
	 * Attach a tag to the given binding. If the tag name has not been seen
	 * before, it is registered in the MovieScene's tag list automatically.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Tags", meta=(ScriptMethod))
	static UE_API void TagBinding(const FMovieSceneBindingProxy& Binding, FName TagName);

	/** Remove a tag from the given binding. No-op if the binding lacks the tag. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Tags", meta=(ScriptMethod))
	static UE_API void UntagBinding(const FMovieSceneBindingProxy& Binding, FName TagName);

	/**
	 * Remove a tag entirely from the sequence. Clears the tag from every
	 * binding that had it, and removes the registration.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Tags", meta=(ScriptMethod))
	static UE_API void RemoveBindingTag(UMovieSceneSequence* Sequence, FName TagName);
};

#undef UE_API
