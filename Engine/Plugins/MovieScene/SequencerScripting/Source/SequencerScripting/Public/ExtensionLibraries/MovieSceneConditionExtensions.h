// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneConditionExtensions.generated.h"

class UMovieSceneSection;
class UMovieSceneTrack;
class UMovieSceneCondition;

#define UE_API SEQUENCERSCRIPTING_API

// Extension methods for getting and setting conditions on sections and tracks
UCLASS()
class UMovieSceneConditionExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// Get the condition on this section, or nullptr if none is set
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section", meta=(ScriptMethod))
	static UE_API UMovieSceneCondition* GetSectionCondition(UMovieSceneSection* Section);

	// Set a condition on this section by creating a new instance of the given class.
	// Pass nullptr to clear the condition.
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta=(ScriptMethod))
	static UE_API void SetSectionCondition(UMovieSceneSection* Section, TSubclassOf<UMovieSceneCondition> ConditionClass);

	// Remove the condition from this section
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta=(ScriptMethod))
	static UE_API void ClearSectionCondition(UMovieSceneSection* Section);

	// Get the track-level condition, or nullptr if none is set
	UFUNCTION(BlueprintPure, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API UMovieSceneCondition* GetTrackCondition(UMovieSceneTrack* Track);

	// Set a track-level condition by creating a new instance of the given class.
	// Pass nullptr to clear the condition.
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API void SetTrackCondition(UMovieSceneTrack* Track, TSubclassOf<UMovieSceneCondition> ConditionClass);

	// Remove the track-level condition
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API void ClearTrackCondition(UMovieSceneTrack* Track);

	// Get the condition on a specific track row, or nullptr if none is set
	UFUNCTION(BlueprintPure, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API UMovieSceneCondition* GetTrackRowCondition(UMovieSceneTrack* Track, int32 RowIndex);

	// Set a condition on a specific track row by creating a new instance of the given class.
	// Pass nullptr to clear the row condition.
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API void SetTrackRowCondition(UMovieSceneTrack* Track, int32 RowIndex, TSubclassOf<UMovieSceneCondition> ConditionClass);

	// Remove the condition from a specific track row
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API void ClearTrackRowCondition(UMovieSceneTrack* Track, int32 RowIndex);
};

#undef UE_API
