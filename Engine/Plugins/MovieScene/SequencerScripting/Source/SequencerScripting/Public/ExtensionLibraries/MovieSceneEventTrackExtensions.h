// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "Channels/MovieSceneEvent.h"

#include "MovieSceneEventTrackExtensions.generated.h"

class UMovieSceneEventTrack;
class UMovieSceneEventRepeaterSection;
class UMovieSceneEventTriggerSection;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto UMovieSceneEventTrack for scripting
 */
UCLASS()
class UMovieSceneEventTrackExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Create a new event repeater section for the given track
	 *
	 * @param Track        The event track to add the new event repeater section to
	 * @return The newly created event repeater section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API UMovieSceneEventRepeaterSection* AddEventRepeaterSection(UMovieSceneEventTrack* InTrack);
	
	/**
	 * Create a new event trigger section for the given track
	 *
	 * @param Track        The event track to add the new event trigger section to
	 * @return The newly created event trigger section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API UMovieSceneEventTriggerSection* AddEventTriggerSection(UMovieSceneEventTrack* InTrack);
	
	/*
	 * Return the class of the bound object property
	 *
	 * @param EventKey    The event key to get the bound object property from
	 * @return The class of the bound object property
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Event", meta = (ScriptMethod))
	static UClass* GetBoundObjectPropertyClass(const FMovieSceneEvent& EventKey) { return EventKey.GetBoundObjectPropertyClass(); }
};

#undef UE_API
