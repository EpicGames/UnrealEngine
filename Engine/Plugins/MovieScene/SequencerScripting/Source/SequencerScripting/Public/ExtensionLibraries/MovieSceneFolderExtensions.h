// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneFolderExtensions.generated.h"

struct FMovieSceneBindingProxy;

class UMovieSceneFolder;
class UMovieSceneTrack;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto UMovieSceneFolders for scripting
 */
UCLASS()
class UMovieSceneFolderExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Get the given folder's display name
	 *
	 * @param Folder	The folder to use
	 * @return The target folder's name
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FName GetFolderName(UMovieSceneFolder* Folder);

	/**
	 * Set the name of the given folder
	 *
	 * @param Folder		The folder to set the name of
	 * @param InFolderName	The new name for the folder
	 * @return True if the setting of the folder name succeeds
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API bool SetFolderName(UMovieSceneFolder* Folder, FName InFolderName);

	/**
	 * Get the display color of the given folder
	 *
	 * @param Folder	The folder to get the display color of
	 * @return The display color of the given folder
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API FColor GetFolderColor(UMovieSceneFolder* Folder);

	/**
	 * Set the display color of the given folder
	 *
	 * @param Folder			The folder to set the display color of
	 * @param InFolderColor		The new display color for the folder
	 * @return True if the folder's display color is set successfully
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API bool SetFolderColor(UMovieSceneFolder* Folder, FColor InFolderColor);

	/**
	 * Get the child folders of a given folder
	 *
	 * @param Folder	The folder to get the child folders of
	 * @return The child folders associated with the given folder
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API TArray<UMovieSceneFolder*> GetChildFolders(UMovieSceneFolder* Folder);

	/**
	 * Add a child folder to the target folder
	 *
	 * @param TargetFolder	The folder to add a child folder to
	 * @param FolderToAdd	The child folder to be added
	 * @return True if the addition is successful
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API bool AddChildFolder(UMovieSceneFolder* TargetFolder, UMovieSceneFolder* FolderToAdd);

	/**
	 * Remove a child folder from the given folder
	 *
	 * @param TargetFolder		The folder from which to remove a child folder
	 * @param FolderToRemove	The child folder to be removed
	 * @return True if the removal succeeds
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API bool RemoveChildFolder(UMovieSceneFolder* TargetFolder, UMovieSceneFolder* FolderToRemove);

	/**
	 * Get the tracks contained by this folder
	 *
	 * @param Folder	The folder to get the tracks of
	 * @return The tracks under the given folder
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API TArray<UMovieSceneTrack*> GetChildTracks(UMovieSceneFolder* Folder);
	
	/**
	 * Add a track to this folder
	 *
	 * @param Folder			The folder to add a child track to
	 * @param InTrack		    The track to add to the folder
	 * @return True if the addition is successful
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API bool AddChildTrack(UMovieSceneFolder* Folder, UMovieSceneTrack* InTrack);
	
	/**
	 * Remove a track from the given folder
	 *
	 * @param Folder			The folder from which to remove a track
	 * @param InTrack		    The track to remove
	 * @return True if the removal succeeds
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API bool RemoveChildTrack(UMovieSceneFolder* Folder, UMovieSceneTrack* InTrack);

	/**
	 * Get the object bindings contained by this folder
	 *
	 * @param Folder	The folder to get the bindings of
	 * @return The object bindings under the given folder
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API TArray<FMovieSceneBindingProxy> GetChildObjectBindings(UMovieSceneFolder* Folder);

	/** 
	 * Add a guid for an object binding to this folder 
	 *
	 * @param Folder			The folder to add a child object to
	 * @param InObjectBinding	The binding to add to the folder
	 * @return True if the addition is successful
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API bool AddChildObjectBinding(UMovieSceneFolder* Folder, FMovieSceneBindingProxy InObjectBinding);

	/** 
	 * Remove an object binding from the given folder
	 *
	 * @param Folder			The folder from which to remove an object binding
	 * @param InObjectBinding	The object binding to remove
	 * @return True if the operation succeeds
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API bool RemoveChildObjectBinding(UMovieSceneFolder* Folder, const FMovieSceneBindingProxy InObjectBinding);
};

#undef UE_API
