// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneMaterialTrackExtensions.generated.h"

class UMovieSceneComponentMaterialTrack;

class UMovieSceneMaterialTrack;

struct FComponentMaterialInfo;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto UMovieSceneMaterialTrack for scripting
 */
UCLASS()
class UMovieSceneMaterialTrackExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Set material index of the component that is animated by the material track.
	 * @param Track The track to use
	 * @param MaterialIndex The desired material index to animate. Values of < 0 or >= NumMaterials will be silently ignored and evaluation will fail.
	 */
	UE_DEPRECATED(5.4, "Use SetMaterialInfo instead")
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod, DeprecatedFunction, DeprecationMessage = "Use SetMaterialInfo instead"))
	static void SetMaterialIndex(UMovieSceneComponentMaterialTrack* Track, const int32 MaterialIndex);

	/**
	 * Get material index of the component that is animated by the material track.
	 * @param Track The track to use
	 * @return The material index.
	 */
	UE_DEPRECATED(5.4, "Use SetMaterialInfo instead")
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DeprecatedFunction, DeprecationMessage = "Use SetMaterialInfo instead"))
	static int32 GetMaterialIndex(UMovieSceneComponentMaterialTrack* Track);

		/**
	 * Set material info of the component that is animated by the material track.
	 * @param Track The track to use
	 * @param MaterialInfo The desired material info to animate.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API void SetMaterialInfo(UMovieSceneComponentMaterialTrack* Track, const FComponentMaterialInfo& MaterialInfo);

	/**
	 * Get material info of the component that is animated by the material track.
	 * @param Track The track to use
	 * @return The material info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DeprecatedFunction, DeprecationMessage = "Use SetMaterialInfo instead"))
	static UE_API FComponentMaterialInfo GetMaterialInfo(UMovieSceneComponentMaterialTrack* Track);
};

#undef UE_API
