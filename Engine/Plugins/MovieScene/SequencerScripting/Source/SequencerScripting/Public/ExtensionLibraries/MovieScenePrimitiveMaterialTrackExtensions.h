// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieScenePrimitiveMaterialTrackExtensions.generated.h"

class UMovieScenePrimitiveMaterialTrack;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto UMovieScenePrimitiveMaterialTrack for scripting
 */
UCLASS()
class UMovieScenePrimitiveMaterialTrackExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Set material index of the element that is animated by the primitive material track.
	 * @param Track The track to use
	 * @param MaterialIndex The desired material index to animate. Values of < 0 or >= NumMaterials will be silently ignored and evaluation will fail.
	 */
	UE_DEPRECATED(5.4, "Use SetMaterialInfo instead")
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod, DeprecatedFunction, DeprecationMessage = "Use SetMaterialInfo instead"))
	static void SetMaterialIndex(UMovieScenePrimitiveMaterialTrack* Track, const int32 MaterialIndex);

	/**
	 * Get material index of the element that is animated by the primitive material track.
	 * @param Track The track to use
	 * @return The material index.
	 */
	UE_DEPRECATED(5.4, "Use SetMaterialInfo instead")
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DeprecatedFunction, DeprecationMessage = "Use SetMaterialInfo instead"))
	static int32 GetMaterialIndex(UMovieScenePrimitiveMaterialTrack* Track);

		/**
	 * Set material info of the component that is animated by the material track.
	 * @param Track The track to use
	 * @param MaterialInfo The desired material info to animate.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API void SetMaterialInfo(UMovieScenePrimitiveMaterialTrack* Track, const FComponentMaterialInfo& MaterialInfo);

	/**
	 * Get material info of the component that is animated by the material track.
	 * @param Track The track to use
	 * @return The material info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DeprecatedFunction, DeprecationMessage = "Use SetMaterialInfo instead"))
	static UE_API FComponentMaterialInfo GetMaterialInfo(UMovieScenePrimitiveMaterialTrack* Track);
};

#undef UE_API
