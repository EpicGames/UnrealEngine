// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieScenePropertyTrackExtensions.generated.h"

class UMovieSceneByteTrack;
class UMovieScenePropertyTrack;
class UMovieSceneObjectPropertyTrack;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto UMovieScenePropertyTrack for scripting
 */
UCLASS()
class UMovieScenePropertyTrackExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Set this track's property name and path
	 *
	 * @param Track        The track to use
	 * @param InPropertyName The property name
	 * @param InPropertyPath The property path
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API void SetPropertyNameAndPath(UMovieScenePropertyTrack* Track, const FName& InPropertyName, const FString& InPropertyPath);

	/**
	 * Get this track's property name
	 *
	 * @param Track        The track to use
	 * @return This track's property name
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API FName GetPropertyName(UMovieScenePropertyTrack* Track);

	/**
	 * Get this track's property path
	 *
	 * @param Track        The track to use
	 * @return This track's property path
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API FString GetPropertyPath(UMovieScenePropertyTrack* Track);

	/**
	 * Get this track's unique name
	 *
	 * @param Track        The track to use
	 * @return This track's unique name
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API FName GetUniqueTrackName(UMovieScenePropertyTrack* Track);

	/**
	 * Set this object property track's property class
	 *
	 * @param Track        The track to use
	 * @param PropertyClass The property class to set
	 * @param bInClassProperty Whether the property class is a class property or not, which determines whether a class property chooser should be used instead of an object property chooser
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API void SetObjectPropertyClass(UMovieSceneObjectPropertyTrack* Track, UClass* PropertyClass, bool bInClassProperty = false);

	/**
	 * Get this object property track's property class
	 *
	 * @param Track        The track to use
	 * @return The property class for this object property track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API UClass* GetObjectPropertyClass(UMovieSceneObjectPropertyTrack* Track);

	/**
	 * Set this byte track's enum
	 *
	 * @param Track        The track to use
	 * @param InEnum The enum to set
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API void SetByteTrackEnum(UMovieSceneByteTrack* Track, UEnum* InEnum);

	/**
	 * Get this byte track's enum
	 *
	 * @param Track        The track to use
	 * @return The enum for this byte track
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API UEnum* GetByteTrackEnum(UMovieSceneByteTrack* Track);
};

#undef UE_API
