// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneSectionEasingExtensions.generated.h"

class UMovieSceneSection;

#define UE_API SEQUENCERSCRIPTING_API

// Function library containing easing methods that should be hoisted onto UMovieSceneSections for scripting
UCLASS()
class UMovieSceneSectionEasingExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	// Get the effective ease-in duration in frames, considering manual override
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section", meta=(ScriptMethod))
	static UE_API int32 GetEaseInDuration(UMovieSceneSection* Section);

	// Get the effective ease-out duration in frames, considering manual override
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section", meta=(ScriptMethod))
	static UE_API int32 GetEaseOutDuration(UMovieSceneSection* Section);

	// Set manual ease-in duration in frames, enabling the manual override
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta=(ScriptMethod))
	static UE_API void SetEaseInDuration(UMovieSceneSection* Section, int32 InDuration);

	// Set manual ease-out duration in frames, enabling the manual override
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta=(ScriptMethod))
	static UE_API void SetEaseOutDuration(UMovieSceneSection* Section, int32 InDuration);
};

#undef UE_API
