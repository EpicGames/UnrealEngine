// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "MovieSceneSectionExtensions.generated.h"

class UMovieSceneScriptingChannel;
class UMovieSceneSequence;
class UMovieSceneSubSection;
struct FSequencerScriptingRange;
struct FMovieSceneChannel;
class UMovieSceneSection;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto UMovieSceneSections for scripting
 */
UCLASS()
class UMovieSceneSectionExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	 * Has start frame
	 *
	 * @param Section        The section being queried
	 * @return Whether this section has a valid start frame (else infinite)
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API bool HasStartFrame(UMovieSceneSection* Section);

	/**
	 * Get start frame. Will throw an exception if section has no start frame, use HasStartFrame to check first.
	 *
	 * @param Section        The section within which to get the start frame
	 * @return Start frame of this section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API int32 GetStartFrame(UMovieSceneSection* Section);

	/**
	 * Get start time in seconds. Will throw an exception if section has no start frame, use HasStartFrame to check first.
	 *
	 * @param Section        The section within which to get the start time
	 * @return Start time of this section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API float GetStartFrameSeconds(UMovieSceneSection* Section);

	/**
	 * Has end frame
	 *
	 * @param Section        The section being queried
	 * @return Whether this section has a valid end frame (else infinite)
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API bool HasEndFrame(UMovieSceneSection* Section);

	/**
	 * Get end frame. Will throw an exception if section has no end frame, use HasEndFrame to check first.
	 *
	 * @param Section        The section within which to get the end frame
	 * @return End frame of this section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API int32 GetEndFrame(UMovieSceneSection* Section);

	/**
	 * Get end time in seconds. Will throw an exception if section has no end frame, use HasEndFrame to check first.
	 *
	 * @param Section        The section within which to get the end time
	 * @return End time of this section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API float GetEndFrameSeconds(UMovieSceneSection* Section);

	/**
	 * Checks to see if this section has an AutoSize implementation, and if so, if that implementation has a start frame.
	 *
	 * @param Section        The section being queried
	 * @return Whether this section has a valid autosize range, and a valid start frame
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API bool GetAutoSizeHasStartFrame(UMovieSceneSection* Section);

	/**
	 * Get start frame of the AutoSize. Will throw an exception if section has no start frame, use GetAutoSizeHasStartFrame to check first.
	 *
	 * @param Section        The section being queried
	 * @return The start frame of the AutoSize data.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API int32 GetAutoSizeStartFrame(UMovieSceneSection* Section);

	/**
	 * Get start time of the AutoSize in seconds. Will throw an exception if section has no start frame, use GetAutoSizeHasStartFrame to check first.
	 *
	 * @param Section        The section being queried
	 * @return The start frame of the AutoSize data in seconds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API float GetAutoSizeStartFrameSeconds(UMovieSceneSection* Section);

	/**
	 * Checks to see if this section has an AutoSize implementation, and if so, if that implementation has a end frame.
	 *
	 * @param Section        The section being queried
	 * @return Whether this section has a valid autosize range, and a valid end frame
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API bool GetAutoSizeHasEndFrame(UMovieSceneSection* Section);

	/**
	 * Get end frame of the AutoSize. Will throw an exception if section has no end frame, use GetAutoSizeHasEndFrame to check first.
	 *
	 * @param Section        The section being queried
	 * @return The end frame of the AutoSize data.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API int32 GetAutoSizeEndFrame(UMovieSceneSection* Section);

	/**
	 * Get end time of the AutoSize seconds. Will throw an exception if section has no end frame, use GetAutoSizeHasEndFrame to check first.
	 *
	 * @param Section        The section being queried
	 * @return The end frame of the AutoSize data in seconds.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API float GetAutoSizeEndFrameSeconds(UMovieSceneSection* Section);

	/**
	 * Set range
	 *
	 * @param Section        The section within which to set the range
	 * @param StartFrame The desired start frame for this section
	 * @param EndFrame The desired end frame for this section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API void SetRange(UMovieSceneSection* Section, int32 StartFrame, int32 EndFrame);

	/**
	 * Set range in seconds
	 *
	 * @param Section        The section within which to set the range
	 * @param StartTime The desired start frame for this section
	 * @param EndTime The desired end frame for this section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API void SetRangeSeconds(UMovieSceneSection* Section, float StartTime, float EndTime);

	/**
	 * Set start frame
	 *
	 * @param Section        The section within which to set the start frame
	 * @param StartFrame The desired start frame for this section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API void SetStartFrame(UMovieSceneSection* Section, int32 StartFrame);

	/**
	 * Set start time in seconds
	 *
	 * @param Section        The section within which to set the start time
	 * @param StartTime The desired start time for this section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API void SetStartFrameSeconds(UMovieSceneSection* Section, float StartTime);

	/**
	 * Set start frame bounded
	 *
	 * @param Section        The section to set whether the start frame is bounded or not
	 * @param IsBounded The desired bounded state of the start frame
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API void SetStartFrameBounded(UMovieSceneSection* Section, bool bIsBounded);

	/**
	 * Set end frame
	 *
	 * @param Section        The section within which to set the end frame
	 * @param EndFrame The desired start frame for this section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API void SetEndFrame(UMovieSceneSection* Section, int32 EndFrame);

	/**
	 * Set end time in seconds
	 *
	 * @param Section        The section within which to set the end time
	 * @param EndTime The desired end time for this section
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API void SetEndFrameSeconds(UMovieSceneSection* Section, float EndTime);

	/**
     * Set end frame bounded
	 *
	 * @param Section        The section to set whether the end frame is bounded or not
	 * @param IsBounded The desired bounded state of the end frame
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API void SetEndFrameBounded(UMovieSceneSection* Section, bool bIsBounded);

	/**
	* Find all channels that belong to the specified UMovieSceneSection. Some sections have many channels (such as
	* Transforms containing 9 double channels to represent Translation/Rotation/Scale), and a section may have mixed
	* channel types.
	*
	* @param Section       The section to use.
	* @return An array containing any key channels that match the type specified
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API TArray<UMovieSceneScriptingChannel*> GetAllChannels(UMovieSceneSection* Section);

	/**
	* Find all channels that belong to the specified UMovieSceneSection that match the specific type. This will filter out any children who do not inherit
	* from the specified type for you.
	*
	* @param Section        The section to use.
	* @param ChannelType	The class type to look for.
	* @return An array containing any key channels that match the type specified
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod, DeterminesOutputType="TrackType", DevelopmentOnly))
	static UE_API TArray<UMovieSceneScriptingChannel*> GetChannelsByType(UMovieSceneSection* Section, TSubclassOf<UMovieSceneScriptingChannel> ChannelType);



	/**
	* Get channel from specified section and channel name
	*
	* @param Section        The section to use.
	* @param ChannelName	The name of the channel.
	* @return The channel if it exists 
	*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod, DeterminesOutputType = "TrackType", DevelopmentOnly))
	static UE_API UMovieSceneScriptingChannel* GetChannel(UMovieSceneSection* Section, const FName& ChannelName);


	/**
	 * Get the frame in the space of its parent sequence
	 *
	 * @param Section        The section that the InFrame is local to
	 * @param InFrame The desired local frame
	 * @param ParentSequence The parent sequence to traverse from
	 * @return The frame at the parent sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section", meta = (ScriptMethod))
	static UE_API int32 GetParentSequenceFrame(UMovieSceneSubSection* Section, int32 InFrame, UMovieSceneSequence* ParentSequence);

public:
	/**
	* Utility function to get channel from a section and a name
	*/
	static UE_API FMovieSceneChannel* GetMovieSceneChannel(UMovieSceneSection* InSection, const FName& InName);

	/**
	* Function to add a channel type for a section
	* @param CustommFunc function to get called when asking for custom channels
	* @return the Guid used to remove it later
	**/
	static UE_API FGuid SetGetCustomChannelsFunction(TFunction<bool(UMovieSceneSection* Section, TSubclassOf<UMovieSceneScriptingChannel> ChannelType, TArray<UMovieSceneScriptingChannel*>& OutChannels)> CustommFunc);

	/**
	* Function to remove custom channels function
	* @param InGuid guid to remove that was returned with SetGetCustomChannelsFunction
	**/
	static UE_API void RemoveGetCustomChannelsFunction(const FGuid& InGuid);

private:

	static TMap<FGuid,TFunction<bool(UMovieSceneSection* Section, TSubclassOf<UMovieSceneScriptingChannel> ChannelType, TArray<UMovieSceneScriptingChannel*>& OutChannels)>> CustomChannelsCallbacks;
};


/**
*   Set of template functions that will help with making custom channel types
*/

template<typename ChannelType, typename ScriptingChannelType>
void SetScriptingChannelHandle(ScriptingChannelType* ScriptingChannel, FMovieSceneChannelProxy& ChannelProxy, int32 ChannelIndex)
{
	ScriptingChannel->ChannelHandle = ChannelProxy.MakeHandle<ChannelType>(ChannelIndex);
}

template<typename ChannelType, typename ScriptingChannelType>
void GetScriptingChannelsForChannel(FMovieSceneChannelProxy& ChannelProxy, TWeakObjectPtr<UMovieSceneSequence> Sequence, TWeakObjectPtr<UMovieSceneSection> Section, TArray<UMovieSceneScriptingChannel*>& OutChannels)
{
	const FMovieSceneChannelEntry* Entry = ChannelProxy.FindEntry(ChannelType::StaticStruct()->GetFName());
	if (Entry)
	{
#if WITH_EDITORONLY_DATA
		TArrayView<const FMovieSceneChannelMetaData> MetaData = Entry->GetMetaData();
		for (int32 Index = 0; Index < MetaData.Num(); ++Index)
		{
			if (MetaData[Index].bEnabled)
			{
				const FName ChannelName = MetaData[Index].Name;
				FName UniqueChannelName = MakeUniqueObjectName(GetTransientPackage(), ScriptingChannelType::StaticClass(), ChannelName);
				ScriptingChannelType* ScriptingChannel = NewObject<ScriptingChannelType>(GetTransientPackage(), UniqueChannelName);
				SetScriptingChannelHandle<ChannelType, ScriptingChannelType>(ScriptingChannel, ChannelProxy, Index);
				ScriptingChannel->OwningSection = Section;
				ScriptingChannel->OwningSequence = Sequence;
				ScriptingChannel->ChannelName = ChannelName;

				OutChannels.Add(ScriptingChannel);
			}
		}
#endif
	}
}

template<typename ChannelType, typename ScriptingChannelType>
UMovieSceneScriptingChannel* GetScriptingChannelForChannel(const FMovieSceneChannel* MovieSceneChannel, FMovieSceneChannelProxy& ChannelProxy, const FMovieSceneChannelHandle& ChanneHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence, TWeakObjectPtr<UMovieSceneSection> Section, const FName& ChannelName)
{
	FName UniqueChannelName = MakeUniqueObjectName(GetTransientPackage(), ScriptingChannelType::StaticClass(), ChannelName);
	ScriptingChannelType* ScriptingChannel = NewObject<ScriptingChannelType>(GetTransientPackage(), UniqueChannelName);
	int32 Index = ChannelProxy.FindIndex(ChanneHandle.GetChannelTypeName(), MovieSceneChannel);
	SetScriptingChannelHandle<ChannelType, ScriptingChannelType>(ScriptingChannel, ChannelProxy, Index);
	ScriptingChannel->OwningSection = Section;
	ScriptingChannel->OwningSequence = Sequence;
	ScriptingChannel->ChannelName = ChannelName;
	return ScriptingChannel;
}

#undef UE_API
