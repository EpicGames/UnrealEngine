// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MovieSceneMarkedFrame.h"
#include "MovieSceneObjectBindingID.h" // for EMovieSceneObjectBindingSpace
#include "MovieSceneTrack.h"
#include "MovieSceneTimeUnit.h"
#include "MovieSceneSequenceExtensions.generated.h"

class UMovieScene;
class UMovieSceneFolder;
class UMovieSceneTrack;
enum class EMovieSceneEvaluationType : uint8;
enum class EUpdateClockSource : uint8;
struct FFrameRate;
struct FMovieSceneBindingProxy;
struct FSequencerScriptingRange;
struct FTimecode;
template <typename T> class TSubclassOf;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto UMovieSceneSequences for scripting purposes
 */
UCLASS()
class UMovieSceneSequenceExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/**
	 * Get this sequence's movie scene data
	 *
	 * @param Sequence        The sequence to use
	 * @return This sequence's movie scene data object
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API UMovieScene* GetMovieScene(UMovieSceneSequence* Sequence);

	/**
	 * Get all tracks
	 *
	 * @param Sequence        The sequence to use
	 * @return An array containing all tracks in this sequence
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API TArray<UMovieSceneTrack*> GetTracks(UMovieSceneSequence* Sequence);

	/**
	 * Find all tracks of the specified type
	 *
	 * @param Sequence        The sequence to use
	 * @param TrackType     A UMovieSceneTrack class type specifying which types of track to return
	 * @return An array containing any tracks that match the type specified
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DeterminesOutputType = "TrackType"))
	static UE_API TArray<UMovieSceneTrack*> FindTracksByType(UMovieSceneSequence* Sequence, TSubclassOf<UMovieSceneTrack> TrackType);

	/**
	 * Find all tracks of the specified type, not allowing sub-classed types
	 *
	 * @param Sequence        The sequence to use
	 * @param TrackType     A UMovieSceneTrack class type specifying the exact types of track to return
	 * @return An array containing any tracks that are exactly the same as the type specified
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod, DeterminesOutputType="TrackType"))
	static UE_API TArray<UMovieSceneTrack*> FindTracksByExactType(UMovieSceneSequence* Sequence, TSubclassOf<UMovieSceneTrack> TrackType);

	/**
	 * Add a new track of the specified type
	 *
	 * @param Sequence        The sequence to use
	 * @param TrackType     A UMovieSceneTrack class type to create
	 * @return The newly created track, if successful
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DeterminesOutputType = "TrackType"))
	static UE_API UMovieSceneTrack* AddTrack(UMovieSceneSequence* Sequence, TSubclassOf<UMovieSceneTrack> TrackType);

	/**
	 * Removes a track
	 *
	 * @param Sequence        The sequence to use
	 * @param Track           The track to remove
	 * @return Whether the track was successfully removed
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API bool RemoveTrack(UMovieSceneSequence* Sequence, UMovieSceneTrack* Track);

	/**
	 * Gets this sequence's display rate
	 *
	 * @param Sequence        The sequence to use
	 * @return The display rate that this sequence is displayed as
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FFrameRate GetDisplayRate(UMovieSceneSequence* Sequence);

	/**
	 * Sets this sequence's display rate
	 *
	 * @param Sequence        The sequence to use
	 * @param DisplayRate The display rate that this sequence is displayed as
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void SetDisplayRate(UMovieSceneSequence* Sequence, FFrameRate DisplayRate);

	/**
	 * Gets this sequence's tick resolution
	 *
	 * @param Sequence        The sequence to use
	 * @return The tick resolution of the sequence, defining the smallest unit of time representable on this sequence
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FFrameRate GetTickResolution(UMovieSceneSequence* Sequence);

	/**
	 * Sets this sequence's tick resolution and migrates frame times
	 *
	 * @param Sequence        The sequence to use
	 * @param TickResolution The tick resolution of the sequence, defining the smallest unit of time representable on this sequence
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void SetTickResolution(UMovieSceneSequence* Sequence, FFrameRate TickResolution);

	/**
	 * Sets this sequence's tick resolution directly without migrating frame times
	 *
	 * @param Sequence        The sequence to use
	 * @param TickResolution The tick resolution of the sequence, defining the smallest unit of time representable on this sequence
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void SetTickResolutionDirectly(UMovieSceneSequence* Sequence, FFrameRate TickResolution);

	/**
	 * Make a new range for this sequence in its display rate
	 *
	 * @param Sequence        The sequence within which to find the binding
	 * @param StartFrame      The frame at which to start the range
	 * @param Duration        The length of the range
	 * @return Specified sequencer range
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FSequencerScriptingRange MakeRange(UMovieSceneSequence* Sequence, int32 StartFrame, int32 Duration);

	/**
	 * Make a new range for this sequence in seconds
	 *
	 * @param Sequence        The sequence within which to find the binding
	 * @param StartTime       The time in seconds at which to start the range
	 * @param Duration        The length of the range in seconds
	 * @return Specified sequencer range
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FSequencerScriptingRange MakeRangeSeconds(UMovieSceneSequence* Sequence, float StartTime, float Duration);

	/**
	 * Get playback range of this sequence in display rate resolution
	 *
	 * @param Sequence        The sequence within which to get the playback range
	 * @return Playback range of this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API FSequencerScriptingRange GetPlaybackRange(UMovieSceneSequence* Sequence);

	/**
	 * Get playback start of this sequence in display rate resolution
	 *
	 * @param Sequence        The sequence within which to get the playback start
	 * @return Playback start of this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API int32 GetPlaybackStart(UMovieSceneSequence* Sequence);

	/**
	 * Get playback start of this sequence in seconds
	 *
	 * @param Sequence        The sequence within which to get the playback start
	 * @return Playback start of this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API float GetPlaybackStartSeconds(UMovieSceneSequence* Sequence);

	/**
	 * Get playback end of this sequence in display rate resolution
	 *
	 * @param Sequence        The sequence within which to get the playback end
	 * @return Playback end of this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API int32 GetPlaybackEnd(UMovieSceneSequence* Sequence);

	/**
	 * Get playback end of this sequence in seconds
	 *
	 * @param Sequence        The sequence within which to get the playback end
	 * @return Playback end of this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API float GetPlaybackEndSeconds(UMovieSceneSequence* Sequence);

	/**
	 * Set playback start of this sequence
	 *
	 * @param Sequence        The sequence within which to set the playback start
	 * @param StartFrame      The desired start frame for this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetPlaybackStart(UMovieSceneSequence* Sequence, int32 StartFrame);

	/**
	 * Set playback start of this sequence in seconds
	 *
	 * @param Sequence        The sequence within which to set the playback start
	 * @param StartTime       The desired start time in seconds for this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetPlaybackStartSeconds(UMovieSceneSequence* Sequence, float StartTime);

	/**
	 * Set playback end of this sequence
	 *
	 * @param Sequence        The sequence within which to set the playback end
	 * @param EndFrame        The desired end frame for this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetPlaybackEnd(UMovieSceneSequence* Sequence, int32 EndFrame);

	/**
	 * Set playback end of this sequence in seconds
	 *
	 * @param Sequence        The sequence within which to set the playback end
	 * @param EndTime         The desired end time in seconds for this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetPlaybackEndSeconds(UMovieSceneSequence* Sequence, float EndTime);

	/**
	 * Set the sequence view range start in seconds
	 *
	 * @param Sequence The sequence within which to set the view range start
	 * @param StartTimeInSeconds The desired view range start time in seconds for this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API void SetViewRangeStart(UMovieSceneSequence* InSequence, double StartTimeInSeconds);

	/**
	 * Get the sequence view range start in seconds
	 *
	 * @param Sequence The sequence within which to get the view range start
	 * @return The view range start time in seconds for this sequence
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API double GetViewRangeStart(UMovieSceneSequence* InSequence);

	/**
	 * Set the sequence view range end in seconds
	 *
	 * @param Sequence The sequence within which to set the view range end
	 * @param StartTimeInSeconds The desired view range end time in seconds for this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API void SetViewRangeEnd(UMovieSceneSequence* InSequence, double EndTimeInSeconds);

	/**
	 * Get the sequence view range end in seconds
	 *
	 * @param Sequence The sequence within which to get the view range end
	 * @return The view range end time in seconds for this sequence
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API double GetViewRangeEnd(UMovieSceneSequence* InSequence);

	/**
	 * Set the sequence work range start in seconds
	 *
	 * @param Sequence The sequence within which to set the work range start
	 * @param StartTimeInSeconds The desired work range start time in seconds for this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API void SetWorkRangeStart(UMovieSceneSequence* InSequence, double StartTimeInSeconds);

	/**
	 * Get the sequence work range start in seconds
	 *
	 * @param Sequence The sequence within which to get the work range start
	 * @return The work range start time in seconds for this sequence
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API double GetWorkRangeStart(UMovieSceneSequence* InSequence);

	/**
	 * Set the sequence work range end in seconds
	 *
	 * @param Sequence The sequence within which to set the work range end
	 * @param StartTimeInSeconds The desired work range end time in seconds for this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API void SetWorkRangeEnd(UMovieSceneSequence* InSequence, double EndTimeInSeconds);

	/**
	 * Get the sequence work range end in seconds
	 *
	 * @param Sequence The sequence within which to get the work range end
	 * @return The work range end time in seconds for this sequence
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Sequence", meta = (ScriptMethod, DevelopmentOnly))
	static UE_API double GetWorkRangeEnd(UMovieSceneSequence* InSequence);

	/**
	 * Set the evaluation type for this sequence
	 *
	 * @param Sequence The sequence within which to set the evaluation type
	 * @param InEvaluationType The evaluation type to set for this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetEvaluationType(UMovieSceneSequence* InSequence, EMovieSceneEvaluationType InEvaluationType);

	/**
	 * Get the evaluation type for this sequence
	 *
	 * @param Sequence The sequence within which to get the evaluation type
	 * @return The evaluation type for this sequence
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API EMovieSceneEvaluationType GetEvaluationType(UMovieSceneSequence* InSequence);

	/**
	 * Set the clock source for this sequence
	 *
	 * @param Sequence The sequence within which to set the clock source
	 * @param InClockSource The clock source to set for this sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetClockSource(UMovieSceneSequence* InSequence, EUpdateClockSource InClockSource);

	/**
	 * Get the clock source for this sequence
	 *
	 * @param Sequence The sequence within which to get the clock source
	 * @return The clock source for this sequence
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API EUpdateClockSource GetClockSource(UMovieSceneSequence* InSequence);

	/**
	 * Get the custom clock for this sequence
	 * 
	 * @param Sequence The Sequence within which to get the custom clock
	 * @return The MovieSceneClock for this sequence
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API UMovieSceneClock* GetCustomClock(UMovieSceneSequence* InSequence);

	/**
	 * Attempt to locate a binding in this sequence by its name
	 *
	 * @param Sequence        The sequence within which to find the binding
	 * @param Name            The display name of the binding to look up
	 * @return A unique identifier for the binding, or invalid
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FMovieSceneBindingProxy FindBindingByName(UMovieSceneSequence* Sequence, FString Name);

	/**
	 * Attempt to locate a binding in this sequence by its Id
	 *
	 * @param Sequence        The sequence within which to find the binding
	 * @param BindingId       The binding Id to look up
	 * @return A unique identifier for the binding, or invalid
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FMovieSceneBindingProxy FindBindingById(UMovieSceneSequence* Sequence, FGuid BindingId);

	/**
	 * Get all the bindings in this sequence
	 *
	 * @param Sequence        The sequence to get bindings for
	 * @return An array of unique identifiers for all the bindings in this sequence
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API TArray<FMovieSceneBindingProxy> GetBindings(UMovieSceneSequence* Sequence);

	/**
	* Get all the spawnables in this sequence. For Level Sequences, this includes bindings with binding type UMovieSceneSpawnableActorBinding.
	*
	* @param Sequence        The sequence to get spawnables for
	* @return Spawnables in this sequence
	*/
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API TArray<FMovieSceneBindingProxy> GetSpawnables(UMovieSceneSequence* Sequence);

	/**
	* Get all the possessables in this sequence. It is understood for the purpose of this function that this means the bindings are not custom.
	*
	* @param Sequence        The sequence to get possessables for
	* @return Possessables in this sequence
	*/
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API TArray<FMovieSceneBindingProxy> GetPossessables(UMovieSceneSequence* Sequence);

	/**
	 * Add a new binding to this sequence that will possess the specified object
	 *
	 * @param Sequence        The sequence to add a possessable to
	 * @param ObjectToPossess The object that this sequence should possess when evaluating
	 * @return A unique identifier for the new binding
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API FMovieSceneBindingProxy AddPossessable(UMovieSceneSequence* Sequence, UObject* ObjectToPossess);

	/**
	 * Add a new binding to this sequence that will spawn the specified object.
	 * For level sequences this will make a custom binding of type UMovieSceneSpawnableActorBinding.
	 *
	 * @param Sequence        The sequence to add to
	 * @param ObjectToSpawn   An object instance to use as a template for spawning
	 * @return A unique identifier for the new binding
	 */
	UE_DEPRECATED(5.5, "AddSpawnableFromInstance is deprecated. Please use AddSpawnableFromInstance in the Level Sequence Editor Subsystem")
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod, DeprecatedFunction, DeprecationMessage = "AddSpawnableFromInstance is deprecated. Please use AddSpawnableFromInstance in the Level Sequence Editor Subsystem"))
	static FMovieSceneBindingProxy AddSpawnableFromInstance(UMovieSceneSequence* Sequence, UObject* ObjectToSpawn);

	/**
	 * Add a new binding to this sequence that will spawn the specified object
	 * For level sequences this will make a custom binding of type UMovieSceneSpawnableActorBinding.
	 *
	 * @param Sequence        The sequence to add to
	 * @param ClassToSpawn    A class or blueprint type to spawn for this binding
	 * @return A unique identifier for the new binding
	 */
	UE_DEPRECATED(5.5, "AddSpawnableFromClass is deprecated. Please use AddSpawnableFromClass in the Level Sequence Editor Subsystem")
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod, DeprecatedFunction, DeprecationMessage = "AddSpawnableFromClass is deprecated. Please use AddSpawnableFromClass in the Level Sequence Editor Subsystem"))
	static FMovieSceneBindingProxy AddSpawnableFromClass(UMovieSceneSequence* Sequence, UClass* ClassToSpawn);

	/**
	 * Locate all the objects that correspond to the specified object ID, using the specified context
	 *
	 * @param Sequence   The sequence to locate bound objects for
	 * @param InBinding  The object binding
	 * @param Context    Optional context to use to find the required object
	 * @return An array of all bound objects
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API TArray<UObject*> LocateBoundObjects(UMovieSceneSequence* Sequence, const FMovieSceneBindingProxy& InBinding, UObject* Context);


	/**
	 * Get the binding ID for a binding within a sequence.
	 * @note: The resulting binding is only valid when applied to properties within the same sequence as this binding. Use GetPortableBindingID for bindings which live in different sub-sequences.
	 *
	 * @param Binding The binding proxy to generate the binding id from
	 * @return The binding's id
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API FMovieSceneObjectBindingID GetBindingID(UMovieSceneSequence* Sequence, const FMovieSceneBindingProxy& InBinding);


	/**
	 * Get a portable binding ID for a binding that resides in a different sequence to the one where this binding will be resolved.
	 * @note: This function must be used over GetBindingID when the target binding resides in different shots or sub-sequences.
	 * @note: Only unique instances of sequences within a root sequences are supported
	 *
	 * @param RootSequence The root sequence that contains both the destination sequence (that will resolve the binding ID) and the target sequence that contains the actual binding
	 * @param DestinationSequence The sequence that will own or resolve the resulting binding ID. For example, if the binding ID will be applied to a camera cut section pass the sequence that contains the camera cut track to this parameter.
	 * @param Binding The target binding to create the ID from
	 * @return The binding's id
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API FMovieSceneObjectBindingID GetPortableBindingID(UMovieSceneSequence* RootSequence, UMovieSceneSequence* DestinationSequence, const FMovieSceneBindingProxy& InBinding);

	/**
	 * Make a binding for the given binding ID
	 *
	 * @param RootSequence  The root sequence that contains the sequence
	 * @param ObjectBindingID The object binding id that has the guid and the sequence id
	 * @return The new binding proxy
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API FMovieSceneBindingProxy ResolveBindingID(UMovieSceneSequence* RootSequence, FMovieSceneObjectBindingID InObjectBindingID);


	/**
	 * Get the root folders in the provided sequence
	 *
	 * @param Sequence	The sequence to retrieve folders from
	 * @return The folders contained within the given sequence
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API TArray<UMovieSceneFolder*> GetRootFoldersInSequence(UMovieSceneSequence* Sequence);

	/**
	 * Add a root folder to the given sequence
	 *
	 * @param Sequence			The sequence to add a folder to
	 * @param NewFolderName		The name to give the added folder
	 * @return The newly created folder
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API UMovieSceneFolder* AddRootFolderToSequence(UMovieSceneSequence* Sequence, FString NewFolderName);

	/**
	 * Remove a root folder from the given sequence. Will throw an exception if the specified folder is not valid or not a root folder.
	 *
	 * @param Sequence			The sequence That the folder belongs to
	 * @param Folder			The folder to remove
	 */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void RemoveRootFolderFromSequence(UMovieSceneSequence* Sequence, UMovieSceneFolder* Folder);

public:

	/**
	 * Filter the specified array of tracks by class, optionally only matching exact classes
	 *
	 * @param InTracks       The array of tracks to filter
	 * @param DesiredClass   The class to match against
	 * @param bExactMatch    Whether to match sub classes or not
	 * @return A filtered array of tracks
	 */
	static UE_API TArray<UMovieSceneTrack*> FilterTracks(TArrayView<UMovieSceneTrack* const> InTracks, UClass* DesiredClass, bool bExactMatch);

public:

	/*
	 * Get the marked frames for this sequence
	 *
	 * @return Return the user marked frames
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DisplayName = "Get Marked Frames"))
	static UE_API TArray<FMovieSceneMarkedFrame> GetMarkedFramesFromSequence(UMovieSceneSequence* Sequence, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	UE_DEPRECATED(5.4, "GetMarkedFrames is deprecated. Please use GetMarkedFrames that takes a time unit and defaults to display rate instead")
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DeprecatedFunction, DeprecationMessage = "GetMarkedFrames is deprecated. Please use GetMarkedFrames that takes a time unit instead"))
	static TArray<FMovieSceneMarkedFrame> GetMarkedFrames(UMovieSceneSequence* Sequence) { return GetMarkedFramesFromSequence(Sequence, EMovieSceneTimeUnit::TickResolution); }

	/*
	 * Add a given user marked frame.
	 * A unique label will be generated if the marked frame label is empty
	 *
	 * @InMarkedFrame The given user marked frame to add
	 * @return The index to the newly added marked frame
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DisplayName = "Add Marked Frame"))
	static UE_API int32 AddMarkedFrameToSequence(UMovieSceneSequence* Sequence, const FMovieSceneMarkedFrame& InMarkedFrame, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	UE_DEPRECATED(5.4, "AddMarkedFrame is deprecated. Please use AddMarkedFrame that takes a time unit and defaults to display rate instead")
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DeprecatedFunction, DeprecationMessage = "AddMarkedFrame is deprecated. Please use AddMarkedFrame that takes a time unit instead"))
	static int32 AddMarkedFrame(UMovieSceneSequence* Sequence, const FMovieSceneMarkedFrame& InMarkedFrame) { return AddMarkedFrameToSequence(Sequence, InMarkedFrame, EMovieSceneTimeUnit::TickResolution); }

	/*
	 * Sets the frame number for the given marked frame index. Does not maintain sort. Call SortMarkedFrames
	 *
	 * @InMarkIndex The given user marked frame index to edit
	 * @InFrameNumber The frame number to set
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DisplayName = "Set Marked Frame"))
	static UE_API void SetMarkedFrameInSequence(UMovieSceneSequence* Sequence, int32 InMarkIndex, FFrameNumber InFrameNumber, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	UE_DEPRECATED(5.4, "SetMarkedFrame is deprecated. Please use SetMarkedFrame that takes a time unit and defaults to display rate instead")
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DeprecatedFunction, DeprecationMessage = "SetMarkedFrame is deprecated. Please use SetMarkedFrame that takes a time unit instead"))
	static void SetMarkedFrame(UMovieSceneSequence* Sequence, int32 InMarkIndex, FFrameNumber InFrameNumber) { SetMarkedFrameInSequence(Sequence, InMarkIndex, InFrameNumber, EMovieSceneTimeUnit::TickResolution); }

	/*
	 * Delete the user marked frame by index.
	 *
	 * @DeleteIndex The index to the user marked frame to delete
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void DeleteMarkedFrame(UMovieSceneSequence* Sequence, int32 DeleteIndex);

	/*
	 * Delete all user marked frames
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void DeleteMarkedFrames(UMovieSceneSequence* Sequence);

	/*
	 * Sort the marked frames in chronological order
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SortMarkedFrames(UMovieSceneSequence* Sequence);

	/*
	 * Find the user marked frame by label
	 *
	 * @InLabel The label to the user marked frame to find
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API int32 FindMarkedFrameByLabel(UMovieSceneSequence* Sequence, const FString& InLabel);

	/*
	 * Find the user marked frame by frame number
	 *
	 * @InFrameNumber The frame number of the user marked frame to find
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DisplayName = "Find Marked Frame By Frame Number"))
	static UE_API int32 FindMarkedFrameByFrameNumberInSequence(UMovieSceneSequence* Sequence, FFrameNumber InFrameNumber, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	UE_DEPRECATED(5.4, "FindMarkedFrameByFrameNumber is deprecated. Please use FindMarkedFrameByFrameNumber that takes a time unit and defaults to display rate instead")
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DeprecatedFunction, DeprecationMessage = "FindMarkedFrameByFrameNumber is deprecated. Please use FindMarkedFrameByFrameNumber that takes a time unit instead"))
	static int32 FindMarkedFrameByFrameNumber(UMovieSceneSequence* Sequence, FFrameNumber InFrameNumber) { return FindMarkedFrameByFrameNumberInSequence(Sequence, InFrameNumber, EMovieSceneTimeUnit::TickResolution); }

	/*
	 * Find the next/previous user marked frame from the given frame number
	 *
	 * @InFrameNumber The frame number to find the next/previous user marked frame from
	 * @bForward Find forward from the given frame number.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DisplayName = "Find Next Marked Frame"))
	static UE_API int32 FindNextMarkedFrameInSequence(UMovieSceneSequence* Sequence, FFrameNumber InFrameNumber, bool bForward, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	UE_DEPRECATED(5.4, "FindNextMarkedFrame is deprecated. Please use FindNextMarkedFrame that takes a time unit instead")
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod, DeprecatedFunction, DeprecationMessage = "FindNextMarkedFrame is deprecated. Please use FindNextMarkedFrame that takes a time unit and defaults to display rate instead"))
	static int32 FindNextMarkedFrame(UMovieSceneSequence* Sequence, FFrameNumber InFrameNumber, bool bForward) { return FindNextMarkedFrameInSequence(Sequence, InFrameNumber, bForward, EMovieSceneTimeUnit::TickResolution); }

	/*
	 * Set read only
	 *
	 * @bInReadOnly Whether the movie scene should be read only or not
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetReadOnly(UMovieSceneSequence* Sequence, bool bInReadOnly);

	/*
	 * Is read only
	 *
	 * @return Whether the movie scene is read only or not
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API bool IsReadOnly(UMovieSceneSequence* Sequence);

	/*
	 * Set playback range locked
	 *
	 * @bInLocked Whether the movie scene playback range should be locked
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetPlaybackRangeLocked(UMovieSceneSequence* Sequence, bool bInLocked);

	/*
	 * Is playback ranged locked
	 *
	 * @return Whether the movie scene playback range is locked
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API bool IsPlaybackRangeLocked(UMovieSceneSequence* Sequence);

	/*
	 * Set marked frames locked
	 *
	 * @bInLocked Whether the movie scene marked frames should be locked
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API void SetMarkedFramesLocked(UMovieSceneSequence* Sequence, bool bInLocked);

	/*
	 * Are marked frames locked
	 *
	 * @return Whether the movie scene marked frames are locked
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta = (ScriptMethod))
	static UE_API bool AreMarkedFramesLocked(UMovieSceneSequence* Sequence);
};

#undef UE_API
