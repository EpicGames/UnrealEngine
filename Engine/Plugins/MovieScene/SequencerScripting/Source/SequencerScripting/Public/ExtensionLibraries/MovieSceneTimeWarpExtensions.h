// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MovieSceneTimeWarpExtensions.generated.h"

struct FMovieSceneTimeWarpVariant;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that relate to time-warp within Sequencer
 */
UCLASS()
class UMovieSceneTimeWarpExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Converts a timewarp variant struct to a constant play rate
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To PlayRate", BlueprintAutocast), Category="Sequencer|TimeWarp")
	static UE_API double Conv_TimeWarpVariantToPlayRate(const FMovieSceneTimeWarpVariant& TimeWarp);

	/**
	 * Converts a constant playrate to a timewarp variant
	 */
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To TimeWarp", BlueprintAutocast), Category="Sequencer|TimeWarp")
	static UE_API FMovieSceneTimeWarpVariant Conv_PlayRateToTimeWarpVariant(double ConstantPlayRate);

	/**
	 * Retrieve this timewarp's constant play rate. Will throw an error if the timewarp is not a constant play rate.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod), Category="Sequencer|TimeWarp")
	static UE_API double ToFixedPlayRate(const FMovieSceneTimeWarpVariant& TimeWarp);

	/**
	 * Assign a constant playrate to this timewarp, overwriting any existing timewarp implementation.
	 */
	UFUNCTION(BlueprintPure, meta=(ScriptMethod), Category="Sequencer|TimeWarp")
	static UE_API void SetFixedPlayRate(UPARAM(ref) FMovieSceneTimeWarpVariant& TimeWarp, double FixedPlayRate);

public:

	UFUNCTION(BlueprintPure, Category="Sequencer|TimeWarp")
	static UE_API void BreakTimeWarp(const FMovieSceneTimeWarpVariant& TimeWarp, double& FixedPlayRate);

	UFUNCTION(BlueprintPure, Category="Sequencer|TimeWarp")
	static UE_API FMovieSceneTimeWarpVariant MakeTimeWarp(double FixedPlayRate);
};

#undef UE_API
