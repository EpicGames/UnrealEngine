// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneTrackExtensions.generated.h"

class FText;

class UMovieSceneTrack;
class UMovieSceneSection;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto UMovieSceneTracks for scripting
 */
UCLASS()
class UMovieSceneTrackExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Set this track's display name
	 *
	 * @param Track        The track to use
	 * @param InName The name for this track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API void SetDisplayName(UMovieSceneTrack* Track, const FText& InName);

	/**
	 * Get this track's display name
	 *
	 * @param Track        The track to use
	 * @return This track's display name
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API FText GetDisplayName(UMovieSceneTrack* Track);

	/**
	 * Set this track row's display name
	 *
	 * @param Track        The track to use
	 * @param InName The name for this track
	 * @param RowIndex The row index for the track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API void SetTrackRowDisplayName(UMovieSceneTrack* Track, const FText& InName, int32 RowIndex);

	/**
	 * Get this track row's display name
	 *
	 * @param Track        The track to use
	 * @param RowIndex The row index for the track
	 * @return This track's display name
	 * 
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API FText GetTrackRowDisplayName(UMovieSceneTrack* Track, int32 RowIndex);

	/**
	 * Add a new section to this track
	 *
	 * @param Track        The track to use
	 * @return The newly create section if successful
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API UMovieSceneSection* AddSection(UMovieSceneTrack* Track);

	/**
	 * Access all this track's sections
	 *
	 * @param Track        The track to use
	 * @return An array of this track's sections
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API TArray<UMovieSceneSection*> GetSections(UMovieSceneTrack* Track);

	/**
	 * Remove the specified section
	 *
	 * @param Track        The track to remove the section from, if present
	 * @param Section      The section to remove
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod))
	static UE_API void RemoveSection(UMovieSceneTrack* Track, UMovieSceneSection* Section);

	/**
	 * Get the sorting order for this track
	 *
	 * @param Track        The track to get the sorting order from
	 * @return The sorting order of the requested track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API int32 GetSortingOrder(UMovieSceneTrack* Track);
 
	/**
	 * Set the sorting order for this track
	 *
	 * @param Track        The track to get the sorting order from
	 * @param SortingOrder The sorting order to set
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API void SetSortingOrder(UMovieSceneTrack* Track, int32 SortingOrder);
 
	/**
	 * Get the color tint for this track
	 *
	 * @param Track        The track to get the color tint from
	 * @return The color tint of the requested track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API FColor GetColorTint(UMovieSceneTrack* Track);
 
	/**
	 * Set the color tint for this track
	 *
	 * @param Track        The track to set the color tint for
	 * @param ColorTint The color tint to set
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API void SetColorTint(UMovieSceneTrack* Track, const FColor& ColorTint);

	/**
	 * Get the section to key for this track
	 *
	 * @param Track        The track to get the section to key for
	 * @return The section to key for the requested track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API UMovieSceneSection* GetSectionToKey(UMovieSceneTrack* Track);
 
	/**
	 * Set the section to key for this track. When properties for this section are modified externally, 
	 * this section will receive those modifications and act accordingly (add/update keys). This is 
	 * especially useful when there are multiple overlapping sections.
	 *
	 * @param Track        The track to set the section to key for
	 * @param Section      The section to key for this track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta=(ScriptMethod, DevelopmentOnly))
	static UE_API void SetSectionToKey(UMovieSceneTrack* Track, UMovieSceneSection* Section);
};

#undef UE_API
