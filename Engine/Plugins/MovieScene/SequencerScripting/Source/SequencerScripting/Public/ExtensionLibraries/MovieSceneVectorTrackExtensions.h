// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneVectorTrackExtensions.generated.h"

class UMovieSceneDoubleVectorTrack;
class UMovieSceneFloatVectorTrack;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto UMovieSceneFloatVectorTrack for scripting
 */
UCLASS()
class UMovieSceneFloatVectorTrackExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Set the number of channels used for this track
	 *
	 * @param Track        The track to set
	 * @param InNumChannelsUsed The number of channels to use for this track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API void SetNumChannelsUsed(UMovieSceneFloatVectorTrack* Track, int32 InNumChannelsUsed);

	/**
	 * Get the number of channels used for this track
	 *
	 * @param Track        The track to query for the number of channels used
	 * @return The number of channels used for this track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API int32 GetNumChannelsUsed(UMovieSceneFloatVectorTrack* Track);
};

/**
 * Function library containing methods that should be hoisted onto UMovieSceneDoubleVectorTrack for scripting
 */
UCLASS()
class UMovieSceneDoubleVectorTrackExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Set the number of channels used for this track
	 *
	 * @param Track        The track to set
	 * @param InNumChannelsUsed The number of channels to use for this track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API void SetNumChannelsUsed(UMovieSceneDoubleVectorTrack* Track, int32 InNumChannelsUsed);

	/**
	 * Get the number of channels used for this track
	 *
	 * @param Track        The track to query for the number of channels used
	 * @return The number of channels used for this track
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Track", meta = (ScriptMethod))
	static UE_API int32 GetNumChannelsUsed(UMovieSceneDoubleVectorTrack* Track);
};

#undef UE_API

