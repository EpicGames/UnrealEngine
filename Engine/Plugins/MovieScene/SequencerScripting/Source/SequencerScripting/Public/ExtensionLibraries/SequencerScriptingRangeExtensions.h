// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "SequencerScriptingRangeExtensions.generated.h"

struct FSequencerScriptingRange;

#define UE_API SEQUENCERSCRIPTING_API

/**
 * Function library containing methods that should be hoisted onto FSequencerScriptingRanges
 */
UCLASS()
class USequencerScriptingRangeExtensions : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	/**
	 * Check whether this range has a start
	 *
	 * @param Range       The range to check
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API bool HasStart(const FSequencerScriptingRange& Range);

	/**
	 * Check whether this range has an end
	 *
	 * @param Range       The range to check
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API bool HasEnd(const FSequencerScriptingRange& Range);

	/**
	 * Remove the start from this range, making it infinite
	 *
	 * @param Range       The range to remove the start from
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void RemoveStart(UPARAM(ref) FSequencerScriptingRange& Range);

	/**
	 * Remove the end from this range, making it infinite
	 *
	 * @param Range       The range to remove the end from
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void RemoveEnd(UPARAM(ref) FSequencerScriptingRange& Range);

	/**
	 * Get the starting time for the specified range in seconds, if it has one. Defined as the first valid time that is inside the range.
	 *
	 * @param Range       The range to get the start from
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API float GetStartSeconds(const FSequencerScriptingRange& Range);

	/**
	 * Get the ending time for the specified range in seconds, if it has one. Defined as the first time that is outside of the range.
	 *
	 * @param Range       The range to get the end from
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API float GetEndSeconds(const FSequencerScriptingRange& Range);

	/**
	 * Set the starting time for the specified range in seconds. Interpreted as the first valid time that is inside the range.
	 *
	 * @param Range       The range to set the start on
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void SetStartSeconds(UPARAM(ref) FSequencerScriptingRange& Range, float Start);

	/**
	 * Set the ending time for the specified range in seconds. Interpreted as the first time that is outside of the range.
	 *
	 * @param Range       The range to set the end on
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void SetEndSeconds(UPARAM(ref) FSequencerScriptingRange& Range, float End);

	/**
	 * Get the starting frame for the specified range, if it has one. Defined as the first valid frame that is inside the range.
	 *
	 * @param Range       The range to get the start from
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API int32 GetStartFrame(const FSequencerScriptingRange& Range);

	/**
	 * Get the ending frame for the specified range, if it has one. Defined as the first subsequent frame that is outside of the range.
	 *
	 * @param Range       The range to get the end from
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API int32 GetEndFrame(const FSequencerScriptingRange& Range);

	/**
	 * Set the starting frame for the specified range. Interpreted as the first valid frame that is inside the range.
	 *
	 * @param Range       The range to set the start on
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void SetStartFrame(UPARAM(ref) FSequencerScriptingRange& Range, int32 Start);

	/**
	 * Set the ending frame for the specified range. Interpreted as the first subsequent frame that is outside of the range.
	 *
	 * @param Range       The range to set the end on
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Sequence", meta=(ScriptMethod))
	static UE_API void SetEndFrame(UPARAM(ref) FSequencerScriptingRange& Range, int32 End);
};

#undef UE_API
