// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "SequencerScriptingRange.h"
#include "MovieSceneTimeUnit.h"
#include "ExtensionLibraries/MovieSceneSequenceExtensions.h"
#include "IMovieSceneRetimingInterface.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "MovieScene.h"
#include "MovieSceneScriptingChannel.generated.h"

#define UE_API SEQUENCERSCRIPTING_API

UCLASS(abstract, BlueprintType, MinimalAPI)
class UMovieSceneScriptingKey : public UObject
{
	GENERATED_BODY()

public:
	/**

	* Gets the time for this key from the owning channel.
	* @param TimeUnit (Optional) Should the time be returned in Display Rate frames (possibly with a sub-frame value) or in Tick Resolution with no sub-frame values? Defaults to Display Rate.
	* @return					 The FrameTime of this key which combines both the frame number and the sub-frame it is on. Sub-frame will be zero if you request Tick Resolution.
	*/
	virtual FFrameTime GetTime(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate) const PURE_VIRTUAL(UMovieSceneScriptingKey::GetTime, return FFrameTime(););
public:
	FKeyHandle KeyHandle;
	TWeakObjectPtr<UMovieSceneSequence> OwningSequence;
	TWeakObjectPtr<UMovieSceneSection> OwningSection;
};


UCLASS(abstract, BlueprintType, MinimalAPI)
class UMovieSceneScriptingChannel : public UObject
{
	GENERATED_BODY()

public:
	/**
	* Gets all of the keys in this channel.
	* @return	An array of UMovieSceneScriptingKey's contained by this channel.
	*			Returns all keys even if clipped by the owning section's boundaries or outside of the current sequence play range.
	*/
	virtual TArray<UMovieSceneScriptingKey*> GetKeys() const PURE_VIRTUAL(UMovieSceneScriptingChannel::GetKeys, return TArray<UMovieSceneScriptingKey*>(););

	/**
	* Gets the keys in this channel specified by the specific index
	* @Indices  The indices from which to get the keys from
	* @return	An array of UMovieSceneScriptingKey's contained by this channel.
	*			Returns all keys specified by the indices, even if out of range.
	*/
	virtual TArray<UMovieSceneScriptingKey*> GetKeysByIndex(const TArray<int32> &Indices) const PURE_VIRTUAL(UMovieSceneScriptingChannel::GetKeysByIndex, return TArray<UMovieSceneScriptingKey*>(););


	UPROPERTY(BlueprintReadOnly, Category="Sequencer|Keys")
	FName ChannelName;
};

template<typename ChannelType, typename ScriptingKeyType, typename ScriptingKeyValueType>
struct TMovieSceneScriptingChannel
{
	static ScriptingKeyType* AddKeyInChannel(TMovieSceneChannelHandle<ChannelType> ChannelHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence, 
		TWeakObjectPtr<UMovieSceneSection> Section, const FFrameNumber InTime, ScriptingKeyValueType& NewValue, float SubFrame, EMovieSceneTimeUnit TimeUnit, EMovieSceneKeyInterpolation Interpolation)
	{
		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			ScriptingKeyType* Key = NewObject<ScriptingKeyType>();

			// Clamp sub-frames to 0-1
			SubFrame = FMath::Clamp(SubFrame, 0.f, FFrameTime::MaxSubframe);

			// TickResolution doesn't support a sub-frame as you can't get finer detailed than that.
			if (TimeUnit == EMovieSceneTimeUnit::TickResolution && SubFrame > 0.f)
			{
				FFrame::KismetExecutionMessage(TEXT("SetTime called with a SubFrame specified for a Tick Resolution type time! SubFrames are only allowed for Display Rate types, ignoring..."), ELogVerbosity::Warning);
				SubFrame = 0.f;
			}

			// The Key's time is always going to be in Tick Resolution space, but the user may want to set it via Display Rate, so we convert.
			FFrameNumber KeyTime = InTime;
			if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
			{
				KeyTime = FFrameRate::TransformTime(FFrameTime(KeyTime, SubFrame), UMovieSceneSequenceExtensions::GetDisplayRate(Sequence.Get()), UMovieSceneSequenceExtensions::GetTickResolution(Sequence.Get())).RoundToFrame();
			}

			if (Section.IsValid())
			{
				Section->Modify();
			}

			using namespace UE::MovieScene;
			Key->KeyHandle = AddKeyToChannel(Channel, KeyTime, NewValue, Interpolation);
			Key->ChannelHandle = ChannelHandle;
			Key->OwningSection = Section;
			Key->OwningSequence = Sequence;

#if WITH_EDITOR
			const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
			if (MetaData && Section.IsValid() && Sequence.IsValid() && Sequence->GetMovieScene())
			{
				TArray<FFrameNumber> Frames;
				Frames.Add(KeyTime);
				Sequence->GetMovieScene()->OnChannelChangedWithTime().Broadcast(Frames, MetaData, Section.Get());
			}
#endif

			return Key;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to add key."), ELogVerbosity::Error);
		return nullptr;
	}

	static void RemoveKeyFromChannel(TMovieSceneChannelHandle<ChannelType> ChannelHandle, UMovieSceneScriptingKey* Key)
	{
		if (!Key)
		{
			return;
		}

		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			if (Key->OwningSection.IsValid())
			{
				Key->OwningSection->Modify();
			}
			
			FFrameTime CurrentTime = Key->GetTime(EMovieSceneTimeUnit::TickResolution);

			TArrayView<FKeyHandle, int> KeyArray = TArrayView<FKeyHandle, int>(&Key->KeyHandle, 1);
			Channel->DeleteKeys(KeyArray);
#if WITH_EDITOR
			const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
			if (MetaData && Key->OwningSection.IsValid() && Key->OwningSequence.IsValid() && Key->OwningSequence->GetMovieScene())
			{
				TArray<FFrameNumber> Frames;
				Frames.Add(CurrentTime.GetFrame());
				Key->OwningSequence->GetMovieScene()->OnChannelChangedWithTime().Broadcast(Frames, MetaData, Key->OwningSection.Get());
			}
#endif
			return;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to remove key."), ELogVerbosity::Error);
	}

	static TArray<UMovieSceneScriptingKey*> GetKeysInChannelByIndex(TMovieSceneChannelHandle<ChannelType> ChannelHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence, TWeakObjectPtr<UMovieSceneSection> Section,
		const TArray<int32>& Indices)
	{
		TArray<UMovieSceneScriptingKey*> OutScriptingKeys;
		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			TArray<FFrameNumber> OutTimes;
			TArray<FKeyHandle> OutKeys;
			Channel->GetKeys(TRange<FFrameNumber>(), &OutTimes, &OutKeys);

			for (int32 Index: Indices)
			{
				if (Index >= 0 && Index < OutTimes.Num())
				{
					ScriptingKeyType * Key = NewObject<ScriptingKeyType>();
					Key->KeyHandle = OutKeys[Index];
					Key->ChannelHandle = ChannelHandle;
					Key->OwningSequence = Sequence;
					Key->OwningSection = Section;
					OutScriptingKeys.Add(Key);
				}
				else
				{
					FFrame::KismetExecutionMessage(TEXT("Invalid index for MovieSceneScriptingChannel, failed to get keys by index."), ELogVerbosity::Error);
				}
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to get keys."), ELogVerbosity::Error);
		}

		return OutScriptingKeys;
	}

	static TArray<UMovieSceneScriptingKey*> GetKeysInChannel(TMovieSceneChannelHandle<ChannelType> ChannelHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence, TWeakObjectPtr<UMovieSceneSection> Section)
	{
		TArray<UMovieSceneScriptingKey*> OutScriptingKeys;
		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			TArray<FFrameNumber> OutTimes;
			TArray<FKeyHandle> OutKeys;
			Channel->GetKeys(TRange<FFrameNumber>(), &OutTimes, &OutKeys);

			for (int32 i = 0; i < OutTimes.Num(); i++)
			{
				ScriptingKeyType* Key = NewObject<ScriptingKeyType>();
				Key->KeyHandle = OutKeys[i];
				Key->ChannelHandle = ChannelHandle;
				Key->OwningSequence = Sequence;
				Key->OwningSection = Section;
				OutScriptingKeys.Add(Key);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to get keys."), ELogVerbosity::Error);
		}

		return OutScriptingKeys;
	}

	static TArray<ScriptingKeyValueType> EvaluateKeysInChannel(TMovieSceneChannelHandle<ChannelType> ChannelHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence, FSequencerScriptingRange ScriptingRange, FFrameRate FrameRate)
	{
		TArray<ScriptingKeyValueType> OutValues;
		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			UMovieSceneSequence* MovieSceneSequence = Sequence.Get();
			FFrameRate Resolution = UMovieSceneSequenceExtensions::GetTickResolution(MovieSceneSequence);
			TRange<FFrameNumber> SpecifiedRange = ScriptingRange.ToNative(Resolution);
			if (SpecifiedRange.HasLowerBound() && SpecifiedRange.HasUpperBound())
			{
				FFrameTime Interval = FFrameRate::TransformTime(1, FrameRate, Resolution);
				FFrameNumber InFrame = UE::MovieScene::DiscreteInclusiveLower(SpecifiedRange);
				FFrameNumber OutFrame = UE::MovieScene::DiscreteExclusiveUpper(SpecifiedRange);
				for (FFrameTime EvalTime = InFrame; EvalTime < OutFrame; EvalTime += Interval)
				{
					FFrameNumber KeyTime = FFrameRate::Snap(EvalTime, Resolution, FrameRate).FloorToFrame();
					Channel->Evaluate(KeyTime, OutValues.Emplace_GetRef());
				}
			}
			else
			{
				FFrame::KismetExecutionMessage(TEXT("Unbounded range passed to evaluate keys."), ELogVerbosity::Error);
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to evaluate keys."), ELogVerbosity::Error);
		}

		return OutValues;
	}

	static FSequencerScriptingRange ComputeEffectiveRangeInChannel(TMovieSceneChannelHandle<ChannelType> ChannelHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence)
	{
		FSequencerScriptingRange ScriptingRange;
		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			ScriptingRange = FSequencerScriptingRange::FromNative(Channel->ComputeEffectiveRange(), UMovieSceneSequenceExtensions::GetTickResolution(Sequence.Get()));
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to get effective range."), ELogVerbosity::Error);
		}

		return ScriptingRange;
	}

	static void SetDefaultInChannel(TMovieSceneChannelHandle<ChannelType> ChannelHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence, TWeakObjectPtr<UMovieSceneSection> Section, ScriptingKeyValueType& InDefaultValue)
	{
		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			if (Section.IsValid())
			{
				Section->Modify();
			}

			using namespace UE::MovieScene;
			SetChannelDefault(Channel, InDefaultValue);

#if WITH_EDITOR
			const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
			if (MetaData && Section.IsValid() && Sequence.IsValid() && Sequence->GetMovieScene())
			{
				TArray<FFrameNumber> Frames;
				Sequence->GetMovieScene()->OnChannelChangedWithTime().Broadcast(Frames, MetaData, Section.Get());
			}
#endif
			return;
		}
		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to set default value."), ELogVerbosity::Error);
	}

	static void RemoveDefaultFromChannel(TMovieSceneChannelHandle<ChannelType> ChannelHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence, TWeakObjectPtr<UMovieSceneSection> Section)
	{
		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			if (Section.IsValid())
			{
				Section->Modify();
			}

			using namespace UE::MovieScene;
			RemoveChannelDefault(Channel);

#if WITH_EDITOR
			const FMovieSceneChannelMetaData* MetaData = ChannelHandle.GetMetaData();
			if (MetaData && Section.IsValid() && Sequence.IsValid() && Sequence->GetMovieScene())
			{
				TArray<FFrameNumber> Frames;
				Sequence->GetMovieScene()->OnChannelChangedWithTime().Broadcast(Frames, MetaData, Section.Get());
			}
#endif
			return;
		}
		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to remove default value."), ELogVerbosity::Error);
	}

	static TOptional<ScriptingKeyValueType> GetDefaultFromChannel(TMovieSceneChannelHandle<ChannelType> ChannelHandle)
	{
		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			ScriptingKeyValueType Ret;

			using namespace UE::MovieScene;
			if (GetChannelDefault(Channel, Ret))
			{
				return TOptional<ScriptingKeyValueType>(Ret);
			}

			return TOptional<ScriptingKeyValueType>();
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to get default value."), ELogVerbosity::Error);
		return TOptional<ScriptingKeyValueType>();
	}

	struct FTransformRetiming : UE::MovieScene::IRetimingInterface
	{
		FTransformRetiming(FFrameNumber InOffsetFrame, double InScale, FFrameNumber InPivotFrame, const TRange<FFrameNumber>& InRange)
			: OffsetFrame(InOffsetFrame)
			, Scale(InScale)
			, PivotFrame(InPivotFrame)
			, Range(InRange) {}

		virtual FFrameTime RemapTime(FFrameTime InTime) const override
		{
			FFrameNumber FrameNumber = InTime.RoundToFrame();

			if (Range.Contains(FrameNumber))
			{
				if (Scale != 0.0)
				{
					FrameNumber = (FrameNumber - PivotFrame) * Scale + PivotFrame;
				}
				return FrameNumber + OffsetFrame;
			}
			return InTime;
		}
		
		double GetScale() const override { return 1.0; }
		TUniquePtr<IRetimingInterface> RecurseInto(UMovieScene* InMovieScene) const override { return nullptr; }
		void Begin(UMovieScene* InMovieScene) const override {}
		void End(UMovieScene* InMovieScene) const override {}

		FFrameNumber OffsetFrame;
		double Scale = 0.0;
		FFrameNumber PivotFrame;
		TRange<FFrameNumber> Range;
	};

	static void TransformKeysInChannel(TMovieSceneChannelHandle<ChannelType> ChannelHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence, TWeakObjectPtr<UMovieSceneSection> Section, FFrameNumber InOffsetFrame, double InScale, FFrameNumber InPivotFrame, FSequencerScriptingRange ScriptingRange, EMovieSceneTimeUnit TimeUnit)
	{
		// The Key's time is always going to be in Tick Resolution space, but the user may want to set it via Display Rate, so we convert.
		FFrameNumber OffsetFrame = InOffsetFrame;
		FFrameNumber PivotFrame = InPivotFrame;

		UMovieSceneSequence* MovieSceneSequence = Sequence.Get();
		FFrameRate Resolution = UMovieSceneSequenceExtensions::GetTickResolution(MovieSceneSequence);
		TRange<FFrameNumber> Range = ScriptingRange.ToNative(Resolution);

		if (Range.IsEmpty())
		{
			Range = TRange<FFrameNumber>::All();
		}

		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			FFrameRate DisplayRate = UMovieSceneSequenceExtensions::GetDisplayRate(Sequence.Get());
			FFrameRate TickResolution = UMovieSceneSequenceExtensions::GetTickResolution(Sequence.Get());

			OffsetFrame = FFrameRate::TransformTime(OffsetFrame, DisplayRate, TickResolution).RoundToFrame();
			PivotFrame = FFrameRate::TransformTime(PivotFrame, DisplayRate, TickResolution).RoundToFrame();

			ScriptingRange.InternalRate = DisplayRate;
			Range = ScriptingRange.ToNative(Resolution);
		}

		if (Section.IsValid())
		{
			Section->Modify();
		}

		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			FTransformRetiming Retimer(OffsetFrame, InScale, PivotFrame, Range);
			Channel->RemapTimes(Retimer);
			return;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingChannel, failed to transform keys in channel."), ELogVerbosity::Error);
	}
};

/** 
* The existing Sequencer code is heavily template-based. We cannot create templated UObjects nor create UFUNCTIONS out template functions.
* This template class serves as a way to minimize boilerplate code when creating UObject versions of the Sequencer key data.
*/
template<typename ChannelType, typename ChannelDataType>
struct TMovieSceneScriptingKey
{
	FFrameTime GetTimeFromChannel(FKeyHandle KeyHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence, EMovieSceneTimeUnit TimeUnit) const
	{
		if (!Sequence.IsValid())
		{
			FFrame::KismetExecutionMessage(TEXT("GetTime called with an invalid owning sequence."), ELogVerbosity::Error);
			return FFrameNumber(0);
		}

		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			FFrameNumber KeyTime;
			Channel->GetKeyTime(KeyHandle, KeyTime);

			// The KeyTime is always going to be in Tick Resolution space, but the user may desire it in Play Rate /w a Subframe.
			if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
			{
				FFrameTime DisplayRateTime = FFrameRate::TransformTime(KeyTime, UMovieSceneSequenceExtensions::GetTickResolution(Sequence.Get()), UMovieSceneSequenceExtensions::GetDisplayRate(Sequence.Get()));
				return DisplayRateTime;
			}

			// Tick Resolution has no sub-frame support.
			return FFrameTime(KeyTime, 0.f);
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingKey, failed to retrieve Time."), ELogVerbosity::Error);
		return FFrameTime();
	}

	void SetTimeInChannel(FKeyHandle KeyHandle, TWeakObjectPtr<UMovieSceneSequence> Sequence, TWeakObjectPtr<UMovieSceneSection> Section, const FFrameNumber NewFrameNumber, EMovieSceneTimeUnit TimeUnit, float SubFrame)
	{
		if (!Sequence.IsValid())
		{
			FFrame::KismetExecutionMessage(TEXT("SetTime called with an invalid owning sequence."), ELogVerbosity::Error);
			return;
		}

		// Clamp sub-frames to 0-1
		SubFrame = FMath::Clamp(SubFrame, 0.f, FFrameTime::MaxSubframe);

		// TickResolution doesn't support a sub-frame as you can't get finer detailed than that.
		if (TimeUnit == EMovieSceneTimeUnit::TickResolution && SubFrame > 0.f)
		{
			FFrame::KismetExecutionMessage(TEXT("SetTime called with a SubFrame specified for a Tick Resolution type time! SubFrames are only allowed for Display Rate types, ignoring..."), ELogVerbosity::Warning);
			SubFrame = 0.f;
		}

		FFrameNumber KeyFrameNumber = NewFrameNumber;
		
		// Keys are always stored in Tick Resolution so we need to potentially convert their values.
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			KeyFrameNumber = FFrameRate::TransformTime(FFrameTime(NewFrameNumber, SubFrame), UMovieSceneSequenceExtensions::GetDisplayRate(Sequence.Get()), UMovieSceneSequenceExtensions::GetTickResolution(Sequence.Get())).RoundToFrame();
		}

		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			if (Section.IsValid())
			{
				Section->Modify();
			}

			Channel->SetKeyTime(KeyHandle, KeyFrameNumber);
			return;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingKey, failed to set Time."), ELogVerbosity::Error);
	}
	
	ChannelDataType GetValueFromChannel(FKeyHandle KeyHandle) const
	{
		ChannelDataType Value = ChannelDataType();

		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			using namespace UE::MovieScene;
			if (!GetKeyValue(Channel, KeyHandle, Value))
			{
				FFrame::KismetExecutionMessage(TEXT("Invalid KeyIndex for MovieSceneScriptingKey, failed to get value. Did you forget to create the key through the channel?"), ELogVerbosity::Error);
				return Value;
			}

			return Value;
		}

		FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingKey, failed to get value. Did you forget to create the key through the channel?"), ELogVerbosity::Error);
		return Value;
	}

	void SetValueInChannel(FKeyHandle KeyHandle, TWeakObjectPtr<UMovieSceneSection> Section, ChannelDataType InNewValue)
	{
		ChannelType* Channel = ChannelHandle.Get();
		if (Channel)
		{
			if (Section.IsValid())
			{
				Section->Modify();
			}

			using namespace UE::MovieScene;
			if (!AssignValue(Channel, KeyHandle, InNewValue))
			{
				FFrame::KismetExecutionMessage(TEXT("Invalid KeyIndex for MovieSceneScriptingKey, failed to set value. Did you forget to create the key through the channel?"), ELogVerbosity::Error);
				return;
			}
		}
		else
		{
			FFrame::KismetExecutionMessage(TEXT("Invalid ChannelHandle for MovieSceneScriptingKey, failed to set value. Did you forget to create the key through the channel?"), ELogVerbosity::Error);
		}
	}

public:
	TMovieSceneChannelHandle<ChannelType> ChannelHandle;
};

#undef UE_API
