// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequenceTimeUnit.generated.h"

UENUM()
enum class UE_DEPRECATED(5.4, "ESequenceTimeUnit is deprecated, please use EMovieSceneTimeUnit") ESequenceTimeUnit : uint8
{
	DisplayRate,
	TickResolution
};
