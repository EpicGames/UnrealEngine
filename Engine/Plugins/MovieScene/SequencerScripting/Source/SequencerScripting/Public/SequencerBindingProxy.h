// Copyright Epic Games, Inc. All Rights Reserved.

// HEADER_UNIT_SKIP - Deprecated
/**
 * DEPRECATED MovieSceneBindingProxy.h Use SequencerBindingProxy.h instead
 */
 
#pragma once

UE_DEPRECATED_HEADER(5.1, "Use MovieSceneBindingProxy.h instead of SequencerBindingProxy.h.")

#include "MovieSceneBindingProxy.h"
