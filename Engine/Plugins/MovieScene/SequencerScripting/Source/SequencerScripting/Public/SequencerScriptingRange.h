// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTimeHelpers.h"
#include "Misc/FrameRate.h"

#include "SequencerScriptingRange.generated.h"

class UMovieScene;
class UMovieSceneSequence;


USTRUCT(BlueprintType)
struct FSequencerScriptingRange
{
	GENERATED_BODY()

	FSequencerScriptingRange()
		: bHasStart(0), bHasEnd(0), InclusiveStart(0), ExclusiveEnd(0)
	{}

	static FSequencerScriptingRange FromNative(const TRange<FFrameNumber>& InRange, FFrameRate InputRate)
	{
		FSequencerScriptingRange NewRange;
		NewRange.InternalRate = InputRate;
		NewRange.bHasStart = InRange.GetLowerBound().IsClosed();
		NewRange.bHasEnd   = InRange.GetUpperBound().IsClosed();

		if (NewRange.bHasStart)
		{
			NewRange.InclusiveStart = UE::MovieScene::DiscreteInclusiveLower(InRange).Value;
		}

		if (NewRange.bHasEnd)
		{
			NewRange.ExclusiveEnd = UE::MovieScene::DiscreteExclusiveUpper(InRange).Value;
		}

		return NewRange;
	}

	static FSequencerScriptingRange FromNative(const TRange<FFrameNumber>& InRange, FFrameRate InputRate, FFrameRate InOutputRate)
	{
		FSequencerScriptingRange NewRange;
		NewRange.InternalRate = InOutputRate;
		NewRange.bHasStart = InRange.GetLowerBound().IsClosed();
		NewRange.bHasEnd   = InRange.GetUpperBound().IsClosed();

		if (NewRange.bHasStart)
		{
			NewRange.InclusiveStart = ConvertFrameTime(UE::MovieScene::DiscreteInclusiveLower(InRange), InputRate, InOutputRate).FloorToFrame().Value;
		}

		if (NewRange.bHasEnd)
		{
			NewRange.ExclusiveEnd = ConvertFrameTime(UE::MovieScene::DiscreteExclusiveUpper(InRange), InputRate, InOutputRate).FloorToFrame().Value;
		}

		return NewRange;
	}


	TRange<FFrameNumber> ToNative(FFrameRate OutputRate) const
	{
		TRange<FFrameNumber> Result;

		if (bHasStart)
		{
			const FFrameNumber FrameNumber = ConvertFrameTime(InclusiveStart, InternalRate, OutputRate).FloorToFrame();
			Result.SetLowerBound(TRangeBound<FFrameNumber>::Inclusive(FrameNumber));
		}

		if (bHasEnd)
		{
			const FFrameNumber FrameNumber = ConvertFrameTime(ExclusiveEnd, InternalRate, OutputRate).FloorToFrame();
			Result.SetUpperBound(TRangeBound<FFrameNumber>::Exclusive(FrameNumber));
		}

		return Result;
	}

	UPROPERTY(BlueprintReadWrite, Category=Range, meta=(ScriptName="HasStartValue"))
	uint8 bHasStart : 1;

	UPROPERTY(BlueprintReadWrite, Category=Range, meta=(ScriptName="HasEndValue"))
	uint8 bHasEnd : 1;

	UPROPERTY(BlueprintReadWrite, Category=Range)
	int32 InclusiveStart;

	UPROPERTY(BlueprintReadWrite, Category=Range)
	int32 ExclusiveEnd;

	UPROPERTY()
	FFrameRate InternalRate;
};
