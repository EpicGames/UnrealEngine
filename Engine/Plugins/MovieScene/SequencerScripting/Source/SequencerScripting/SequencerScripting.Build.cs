// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SequencerScripting : ModuleRules
{
	public SequencerScripting(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
            }
		);

		PrivateIncludePaths.AddRange(
			new string[] {
				"SequencerScripting/Private/ExtensionLibraries",
                "SequencerScripting/Private/KeysAndChannels",
            }
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"TimeManagement",
				"MovieScene",
                "MovieSceneTracks",
            }
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
                "LevelSequence",
				"UniversalObjectLocator",
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				
			}
		);
	}
}
