// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerCurveEditorObject.h"
#include "MVVM/CurveEditorExtension.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/ViewModels/OutlinerViewModel.h"
#include "MVVM/Extensions/IOutlinerExtension.h"
#include "MVVM/ViewModels/TrackModel.h"
#include "Tree/SCurveEditorTree.h"
#include "ISequencer.h"
#include "CurveEditor.h"
#include "CurveEditorTypes.h"
#include "CurveModel.h"
#include "Channels/MovieSceneChannel.h"
#include "ExtensionLibraries/MovieSceneSectionExtensions.h"
#include "Filters/CurveEditorFilterBase.h"
#include "ScopedTransaction.h"
#include "Modification/Utils/ScopedCurveChange.h"
#include "Modification/Utils/ScopedSelectionChange.h"
#include "Modification/Keys/CurveChangeDiff.h"

//For custom colors on channels, stored in editor pref's
#include "CurveEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerCurveEditorObject)


TSharedPtr<FCurveEditor> USequencerCurveEditorObject::GetCurveEditor()
{
	using namespace UE::Sequencer;
	if (CurrentSequencer.IsValid())
	{
		const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = CurrentSequencer.Pin()->GetViewModel();
		if (FCurveEditorExtension* CurveEditorExtension = SequencerViewModel->CastDynamic<FCurveEditorExtension>())
		{
			return (CurveEditorExtension->GetCurveEditor());
		}
	}
	return nullptr;
}

void USequencerCurveEditorObject::OpenCurveEditor()
{
	using namespace UE::Sequencer;
	if (CurrentSequencer.IsValid())
	{
		const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = CurrentSequencer.Pin()->GetViewModel();
		if (FCurveEditorExtension* CurveEditorExtension = SequencerViewModel->CastDynamic<FCurveEditorExtension>())
		{
			CurveEditorExtension->OpenCurveEditor();
		}
	}
}

bool USequencerCurveEditorObject::IsCurveEditorOpen()
{
	using namespace UE::Sequencer;
	if (CurrentSequencer.IsValid())
	{
		const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = CurrentSequencer.Pin()->GetViewModel();
		if (const FCurveEditorExtension* CurveEditorExtension = SequencerViewModel->CastDynamic<FCurveEditorExtension>())
		{
			return CurveEditorExtension->IsCurveEditorOpen();
		}
	}
	return false;
}

void USequencerCurveEditorObject::CloseCurveEditor()
{
	using namespace UE::Sequencer;
	if (CurrentSequencer.IsValid())
	{
		const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = CurrentSequencer.Pin()->GetViewModel();
		if (FCurveEditorExtension* CurveEditorExtension = SequencerViewModel->CastDynamic<FCurveEditorExtension>())
		{
			CurveEditorExtension->CloseCurveEditor();
		}
	}
}

TArray<FSequencerChannelProxy> USequencerCurveEditorObject::GetChannelsWithSelectedKeys()
{
	TArray<FSequencerChannelProxy> OutSelectedChannels;
	TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor();
	if (CurrentSequencer.IsValid() && CurveEditor.IsValid())
	{
		const TMap<FCurveModelID, FKeyHandleSet>& SelectionKeyMap = CurveEditor->Selection.GetAll();

		for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : SelectionKeyMap)
		{
			if (const FCurveModel* Curve = CurveEditor->FindCurve(Pair.Key))
			{
				if (UMovieSceneSection* Section = Curve->GetOwningObjectOrOuter<UMovieSceneSection>())
				{
					FName ChannelName = Curve->GetChannelName();
					FSequencerChannelProxy ChannelProxy(ChannelName,Section);
					OutSelectedChannels.Add(ChannelProxy);

				}
			}
		}
	}
	return OutSelectedChannels;
}

TArray<int32> USequencerCurveEditorObject::GetSelectedKeys(const FSequencerChannelProxy& ChannelProxy)
{
	TArray<int32> SelectedKeys;
	TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor();
	if (CurrentSequencer.IsValid() && CurveEditor.IsValid())
	{
		const TMap<FCurveModelID, FKeyHandleSet>& SelectionKeyMap = CurveEditor->Selection.GetAll();

		for (const TPair<FCurveModelID, FKeyHandleSet>& Pair : SelectionKeyMap)
		{
			if (const FCurveModel* Curve = CurveEditor->FindCurve(Pair.Key))
			{
				if (UMovieSceneSection* Section = Curve->GetOwningObjectOrOuter<UMovieSceneSection>())
				{
					if (Section == ChannelProxy.Section)
					{
						if (FMovieSceneChannel* MovieSceneChannel = UMovieSceneSectionExtensions::GetMovieSceneChannel(Section, ChannelProxy.ChannelName))
						{
							TArrayView<const FKeyHandle> HandleArray = Pair.Value.AsArray();
							for (FKeyHandle Key : HandleArray)
							{
								int32 Index = MovieSceneChannel->GetIndex(Key);
								if (Index != INDEX_NONE)
								{
									SelectedKeys.Add(Index);
								}
							}
						}
					}
				}
			}
		}
	}
	return SelectedKeys;
}

void USequencerCurveEditorObject::EmptySelection()
{
	TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor();
	if (CurrentSequencer.IsValid() && CurveEditor.IsValid())
	{
		CurveEditor->Selection.Clear();
	}
}

void USequencerCurveEditorObject::ShowCurve(const FSequencerChannelProxy& ChannelProxy, bool bShowCurve)
{
	using namespace UE::Sequencer;
	TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor();
	if (CurrentSequencer.IsValid() && CurveEditor.IsValid())
	{
		if (IsCurveShown(ChannelProxy) != bShowCurve)
		{
			const TSharedPtr<FSequencerEditorViewModel> SequencerViewModel = CurrentSequencer.Pin()->GetViewModel();
			const FCurveEditorExtension* CurveEditorExtension = SequencerViewModel->CastDynamic<FCurveEditorExtension>();
			check(CurveEditorExtension);
			TSharedPtr<SCurveEditorTree>  CurveEditorTreeView = CurveEditorExtension->GetCurveEditorTreeView();
			TSharedPtr<FOutlinerViewModel> OutlinerViewModel = SequencerViewModel->GetOutliner();
			bool bIsSelected = false;
			TParentFirstChildIterator<IOutlinerExtension> OutlinerExtenstionIt = OutlinerViewModel->GetRootItem()->GetDescendantsOfType<IOutlinerExtension>();
			for (; OutlinerExtenstionIt; ++OutlinerExtenstionIt)
			{
				if (TViewModelPtr<ITrackExtension> TrackModel = OutlinerExtenstionIt.GetCurrentItem()->FindAncestorOfType<ITrackExtension>())
				{
					if (UMovieSceneTrack* Track = TrackModel->GetTrack())
					{
						if (TViewModelPtr<FChannelGroupOutlinerModel> ChannelModel = CastViewModel<FChannelGroupOutlinerModel>(OutlinerExtenstionIt.GetCurrentItem()))
						{
							if (TSharedPtr<FChannelModel> ChannelPtr = ChannelModel->GetChannel(ChannelProxy.Section)) //if not section to key we also don't select it.
							{
								if (ChannelPtr->GetChannelName() == ChannelProxy.ChannelName)
								{
									if (TViewModelPtr<ICurveEditorTreeItemExtension> CurveEditorItem = OutlinerExtenstionIt.GetCurrentItem().ImplicitCast())
									{
										FCurveEditorTreeItemID CurveEditorTreeItem = CurveEditorItem->GetCurveEditorItemID();
										if (CurveEditorTreeItem != FCurveEditorTreeItemID::Invalid())
										{
											CurveEditorTreeView->SetItemSelection(CurveEditorTreeItem, bShowCurve);
											break;
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

bool USequencerCurveEditorObject::IsCurveShown(const FSequencerChannelProxy& ChannelProxy)
{
	TOptional<FCurveModelID> CurveModelID;
	if (UMovieSceneSection* Section = ChannelProxy.Section)
	{
		CurveModelID = USequencerCurveEditorObject::GetCurve(Section, ChannelProxy.ChannelName);
	}
	return CurveModelID.IsSet();
}

TOptional<FCurveModelID> USequencerCurveEditorObject::GetCurve(UMovieSceneSection* InSection, const FName& InName)
{
	TOptional<FCurveModelID> OptCurveModel;
	TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor();
	if (CurrentSequencer.IsValid() && CurveEditor.IsValid())
	{
		const TMap<FCurveModelID, TUniquePtr<FCurveModel>>& Curves = CurveEditor->GetCurves();
		for (const TPair <FCurveModelID, TUniquePtr<FCurveModel>>& Pair : Curves)
		{
			if (Pair.Value.IsValid() && Pair.Value->GetOwningObject() == InSection && Pair.Value->GetChannelName() == InName)
			{
				OptCurveModel = Pair.Key;
				break;
			}
		}
	}
	return OptCurveModel;
}

void USequencerCurveEditorObject::SelectKeys(const FSequencerChannelProxy& ChannelProxy, const TArray<int32>& Indices)
{
	TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor();
	if (CurrentSequencer.IsValid() && CurveEditor.IsValid())
	{
		if (UMovieSceneSection* Section = ChannelProxy.Section)
		{
			TOptional<FCurveModelID> CurveModelID = USequencerCurveEditorObject::GetCurve(Section, ChannelProxy.ChannelName);
			if(CurveModelID.IsSet())
			{
				if (FMovieSceneChannel* MovieSceneChannel = UMovieSceneSectionExtensions::GetMovieSceneChannel(Section, ChannelProxy.ChannelName))
				{
					TArray<FKeyHandle> Handles;
					for (int32 Index : Indices)
					{
						FKeyHandle Handle = MovieSceneChannel->GetHandle(Index);
						if (Handle != FKeyHandle::Invalid())
						{
							Handles.Add(Handle);
						}
					}
					CurveEditor->Selection.Add(CurveModelID.GetValue(), ECurvePointType::Key, Handles);
				}
			}
		}
	}
}

void USequencerCurveEditorObject::SetSequencer(TSharedPtr<ISequencer>& InSequencer)
{
	using namespace UE::Sequencer;
	CurrentSequencer = TWeakPtr<ISequencer>(InSequencer);
}

bool USequencerCurveEditorObject::HasCustomColorForChannel(UClass* Class, const FString& Identifier)
{
	const UCurveEditorSettings* Settings = GetDefault<UCurveEditorSettings>();
	if (Settings)
	{
		TOptional<FLinearColor> OptColor = Settings->GetCustomColor(Class, Identifier);
		return OptColor.IsSet();
	}
	return false;
}

FLinearColor USequencerCurveEditorObject::GetCustomColorForChannel(UClass* Class, const FString& Identifier)
{
	FLinearColor Color(FColor::White);
	const UCurveEditorSettings* Settings = GetDefault<UCurveEditorSettings>();
	if (Settings)
	{
		TOptional<FLinearColor> OptColor = Settings->GetCustomColor(Class, Identifier);
		if (OptColor.IsSet())
		{
			return OptColor.GetValue();
		}
	}
	return Color;
}

void USequencerCurveEditorObject::SetCustomColorForChannel(UClass* Class, const FString& Identifier, const FLinearColor& NewColor)
{
	UCurveEditorSettings* Settings = GetMutableDefault<UCurveEditorSettings>();
	if (Settings)
	{
		Settings->SetCustomColor(Class, Identifier, NewColor);
	}
}

void USequencerCurveEditorObject::SetCustomColorForChannels(UClass* Class, const TArray<FString>& Identifiers, const TArray<FLinearColor>& NewColors)
{
	if (Identifiers.Num() != NewColors.Num())
	{
		return;
	}
	UCurveEditorSettings* Settings = GetMutableDefault<UCurveEditorSettings>();
	if (Settings)
	{
		for (int32 Index = 0; Index < Identifiers.Num(); ++Index)
		{
			const FString& Identifier = Identifiers[Index];
			const FLinearColor& NewColor = NewColors[Index];
			Settings->SetCustomColor(Class, Identifier, NewColor);
		}
	}
}

void USequencerCurveEditorObject::DeleteColorForChannels(UClass* Class, FString& Identifier)
{
	UCurveEditorSettings* Settings = GetMutableDefault<UCurveEditorSettings>();
	if (Settings)
	{
		Settings->DeleteCustomColor(Class, Identifier);
	}
}

void USequencerCurveEditorObject::SetRandomColorForChannels(UClass* Class, const TArray<FString>& Identifiers)
{
	UCurveEditorSettings* Settings = GetMutableDefault<UCurveEditorSettings>();
	if (Settings)
	{
		for (int32 Index = 0; Index < Identifiers.Num(); ++Index)
		{
			const FString& Identifier = Identifiers[Index];
			FLinearColor NewColor = UCurveEditorSettings::GetNextRandomColor();
			Settings->SetCustomColor(Class, Identifier, NewColor);
		}
	}
}

void USequencerCurveEditorObject::ApplyFilter(UCurveEditorFilterBase* Filter)
{
	using namespace UE::CurveEditor;

	TSharedPtr<FCurveEditor> CurveEditor = GetCurveEditor();

	if (!Filter || !CurveEditor.IsValid())
	{
		return;
	}

	const TMap<FCurveModelID, FKeyHandleSet>& SelectedKeys = CurveEditor->GetSelection().GetAll();

	// Build working key set: selection if non-empty, else all visible curves
	TMap<FCurveModelID, FKeyHandleSet> AllCurveKeys;
	const TMap<FCurveModelID, FKeyHandleSet>* KeysToOperateOn = &SelectedKeys;
	const bool bSelectionEmpty = SelectedKeys.IsEmpty();
	if (bSelectionEmpty)
	{
		for (const TTuple<FCurveModelID, TUniquePtr<FCurveModel>>& CurvePair : CurveEditor->GetCurves())
		{
			FKeyHandleSet& HandleSet = AllCurveKeys.Add(CurvePair.Key);
			for (const FKeyHandle& Handle : CurvePair.Value->GetAllKeys())
			{
				HandleSet.Add(Handle, ECurvePointType::Key);
			}
		}
		KeysToOperateOn = &AllCurveKeys;
	}

	const int32 SelectionCount = CurveEditor->GetSelection().Count();
	const FText TransactionText = SelectionCount > 0
		? FText::Format(NSLOCTEXT("CurveEditorFilterApply", "Filter Curves", "Filtered {0}|plural(one=Curve, other=Curves)"), SelectionCount)
		: NSLOCTEXT("CurveEditorFilterApply", "Filter All Visible Curves", "Filtered All Visible Curves");

	const FScopedTransaction Transaction(TransactionText);

	Filter->InitializeFilter(CurveEditor.ToSharedRef());
	TMap<FCurveModelID, FKeyHandleSet> OutKeysToSelect;
	{
		const FScopedCurveChange KeyChange(FCurvesSnapshotBuilder(CurveEditor, *KeysToOperateOn));
		Filter->ApplyFilter(CurveEditor.ToSharedRef(), *KeysToOperateOn, OutKeysToSelect);
	}

	if (bSelectionEmpty == false) //only reselect if we had selection
	{
		const FScopedSelectionChange SelectionTransaction(CurveEditor);

		CurveEditor->GetSelection().Clear();
		for (const TTuple<FCurveModelID, FKeyHandleSet>& OutSet : OutKeysToSelect)
		{
			CurveEditor->GetSelection().Add(OutSet.Key, ECurvePointType::Key, OutSet.Value.AsArray());
		}
	}
}





