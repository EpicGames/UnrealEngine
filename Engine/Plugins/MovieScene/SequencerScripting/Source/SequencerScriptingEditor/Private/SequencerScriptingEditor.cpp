// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerScriptingEditor.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FSequencerScriptingEditorModule"

void FSequencerScriptingEditorModule::StartupModule()
{}

void FSequencerScriptingEditorModule::ShutdownModule()
{}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FSequencerScriptingEditorModule, SequencerScriptingEditor)
