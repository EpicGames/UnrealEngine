// Copyright Epic Games, Inc. All Rights Reserved.

#include "SequencerTools.h"
#include "Animation/AnimSequence.h"
#include "Engine/SkeletalMesh.h"
#include "AutomatedLevelSequenceCapture.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "LevelSequenceActor.h"
#include "Channels/MovieSceneEvent.h"
#include "FbxExporter.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "MovieSceneToolsUserSettings.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "MovieSceneToolHelpers.h"
#include "Engine/World.h"
#include "MovieSceneEventUtils.h"
#include "INodeAndChannelMappings.h"
#include "MovieSceneSequenceEditor.h"
#include "LevelSequence.h"
#include "Sections/MovieSceneEventSectionBase.h"
#include "CineCameraActor.h"
#include "Components/SkeletalMeshComponent.h"
#include "MovieScenePossessable.h"
#include "ScopedTransaction.h"
#include "Exporters/AnimSeqExportOption.h"
#include "TimerManager.h"

#include "K2Node_CustomEvent.h"
#include "BlueprintFunctionNodeSpawner.h"
#include "BlueprintActionMenuItem.h"
#include "EdGraphSchema_K2.h"

#include "LevelSequenceAnimSequenceLink.h"
#include "AnimSequenceLevelSequenceLink.h"

#include "Compilation/MovieSceneCompiledDataManager.h"
#include "MovieSceneSpawnable.h"
#include "SequencerScriptingRange.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SequencerTools)

#define LOCTEXT_NAMESPACE "SequencerTools"

DECLARE_LOG_CATEGORY_CLASS(LogSequencerTools, Log, All);

namespace SequencerToolsPrivate
{
	static TArray<TStrongObjectPtr<const UObject>> DelegateObjectsTempStrongRefs;

	// Will check Condition every Interval seconds (or every frame, if Interval == 0) and execute Callback when true.
	void WaitUntil(const FCheckCanCompleteDelegate Condition, const TFunction<void()> Callback, const UWorld* World, const float Interval = 0.1f)
	{
		if (!Condition.IsBound() || !Callback.IsSet() || !World)
		{
			UE_LOGF(LogSequencerTools, Error, "WaitUntil() - One of the parameters is invalid.");
			return;
		}

		// Guard the condition delegate from GC.
		DelegateObjectsTempStrongRefs.AddUnique(TStrongObjectPtr<const UObject>(Condition.GetUObject()));

		if (Condition.Execute())
		{
			Callback();
			DelegateObjectsTempStrongRefs.Remove(TStrongObjectPtr<const UObject>(Condition.GetUObject()));
			return;
		}

		FTimerDelegate WaitDelegate = FTimerDelegate::CreateLambda([Condition, Callback, World, Interval]() { SequencerToolsPrivate::WaitUntil(Condition, Callback, World, Interval); });
		FTimerManager& TimerManager = World->GetTimerManager();

		if (Interval == 0.f)
		{
			TimerManager.SetTimerForNextTick(WaitDelegate);
		}
		else
		{
			FTimerHandle Handle; // Unnecessary to keep, but required by SetTimer().
			TimerManager.SetTimer(Handle, WaitDelegate, Interval, false);
		}
	}
}


bool USequencerToolsFunctionLibrary::RenderMovie(UMovieSceneCapture* InCaptureSettings, FOnRenderMovieStopped OnFinishedCallback)
{
	IMovieSceneCaptureDialogModule& MovieSceneCaptureModule = FModuleManager::Get().LoadModuleChecked<IMovieSceneCaptureDialogModule>("MovieSceneCaptureDialog");
	
	// Because this comes from the Python/BP layer we need to soft-validate the state before we pass it onto functions that do a assert-based validation.
	if (!InCaptureSettings)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot start Render Sequence to Movie with null capture settings."), ELogVerbosity::Error);
		return false;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (IsRenderingMovie())
	{
		FFrame::KismetExecutionMessage(TEXT("Capture already in progress."), ELogVerbosity::Error);
		return false;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// If they're capturing a level sequence we'll do some additional checking as there are more parameters on the Automated Level Sequence capture.
	UAutomatedLevelSequenceCapture* LevelSequenceCapture = Cast<UAutomatedLevelSequenceCapture>(InCaptureSettings);
	if (LevelSequenceCapture)
	{
		if (!LevelSequenceCapture->LevelSequenceAsset.IsValid())
		{
			// UE_LOGF(LogSequencerTools, Warning, "No Level Sequence Asset specified in UAutomatedLevelSequenceCapture.");
			FFrame::KismetExecutionMessage(TEXT("No Level Sequence Asset specified in UAutomatedLevelSequenceCapture."), ELogVerbosity::Error);
			return false;
		}

		if (!LevelSequenceCapture->bUseCustomStartFrame && !LevelSequenceCapture->bUseCustomEndFrame)
		{
			// If they don't want to use a custom start/end frame we override the default values to be the length of the sequence, as the default is [0,1)
			ULevelSequence* LevelSequence = Cast<ULevelSequence>(LevelSequenceCapture->LevelSequenceAsset.TryLoad());
			if (!LevelSequence)
			{
				const FString ErrorMessage = FString::Printf(TEXT("Specified Level Sequence Asset failed to load. Specified Asset Path: %s"), *LevelSequenceCapture->LevelSequenceAsset.GetAssetPathString());
				FFrame::KismetExecutionMessage(*ErrorMessage, ELogVerbosity::Error);
				return false;
			}

			FFrameRate DisplayRate = LevelSequence->GetMovieScene()->GetDisplayRate();
			FFrameRate TickResolution = LevelSequence->GetMovieScene()->GetTickResolution();
			
			LevelSequenceCapture->Settings.CustomFrameRate = DisplayRate;
			LevelSequenceCapture->Settings.bUseCustomFrameRate = true;
			LevelSequenceCapture->Settings.bUseRelativeFrameNumbers = false;
			TRange<FFrameNumber> Range = LevelSequence->GetMovieScene()->GetPlaybackRange();

			FFrameNumber StartFrame = UE::MovieScene::DiscreteInclusiveLower(Range);
			FFrameNumber EndFrame = UE::MovieScene::DiscreteExclusiveUpper(Range);

			FFrameNumber RoundedStartFrame = FFrameRate::TransformTime(StartFrame, TickResolution, DisplayRate).CeilToFrame();
			FFrameNumber RoundedEndFrame = FFrameRate::TransformTime(EndFrame, TickResolution, DisplayRate).CeilToFrame();

			LevelSequenceCapture->CustomStartFrame = RoundedStartFrame;
			LevelSequenceCapture->CustomEndFrame = RoundedEndFrame;
		}
	}

	auto LocalCaptureStoppedCallback = [OnFinishedCallback](bool bSuccess)
	{
		OnFinishedCallback.ExecuteIfBound(bSuccess);
	};

	MovieSceneCaptureModule.StartCapture(InCaptureSettings);
	MovieSceneCaptureModule.GetCurrentCapture()->CaptureStoppedDelegate.AddLambda(LocalCaptureStoppedCallback);
	return true;
}

void USequencerToolsFunctionLibrary::CancelMovieRender()
{
	IMovieSceneCaptureDialogModule& MovieSceneCaptureModule = FModuleManager::Get().LoadModuleChecked<IMovieSceneCaptureDialogModule>("MovieSceneCaptureDialog");
	TSharedPtr<FMovieSceneCaptureBase> CurrentCapture = MovieSceneCaptureModule.GetCurrentCapture();
	if (CurrentCapture.IsValid())
	{
		// We just invoke the capture's Cancel function. This will cause a shut-down of the capture (the same as the UI)
		// which will invoke all of the necessary callbacks as well. We don't null out CurrentCapture because that is done
		// as the result of its shutdown callbacks.
		CurrentCapture->Cancel();
	}
}

TArray<FSequencerBoundObjects> USequencerToolsFunctionLibrary::GetBoundObjects(UWorld* InWorld, ULevelSequence* InSequence, const TArray<FMovieSceneBindingProxy>& InBindings, const FSequencerScriptingRange& InRange)
{
	ALevelSequenceActor* OutActor;
	FMovieSceneSequencePlaybackSettings Settings;
	FLevelSequenceCameraSettings CameraSettings;

	ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(InWorld, InSequence, Settings, OutActor);

	// Evaluation needs to occur in order to obtain spawnables
	FFrameRate Resolution = InSequence->GetMovieScene()->GetTickResolution();
	TRange<FFrameNumber> SpecifiedRange = InRange.ToNative(Resolution);
	Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(SpecifiedRange.GetLowerBoundValue().Value, EUpdatePositionMethod::Play));

	FMovieSceneSequenceID SequenceId = Player->GetEvaluationState()->FindSequenceId(InSequence);

	TArray<FSequencerBoundObjects> BoundObjects;
	for (FMovieSceneBindingProxy Binding : InBindings)
	{
		FMovieSceneObjectBindingID ObjectBinding = UE::MovieScene::FFixedObjectBindingID(Binding.BindingID, SequenceId);
		BoundObjects.Add(FSequencerBoundObjects(Binding, Player->GetBoundObjects(ObjectBinding)));
	}

	Player->Stop();
	InWorld->DestroyActor(OutActor);
	return BoundObjects;
}


TArray<FSequencerBoundObjects> USequencerToolsFunctionLibrary::GetObjectBindings(UWorld* InWorld, ULevelSequence* InSequence, const TArray<UObject*>& InObjects, const FSequencerScriptingRange& InRange)
{
	ALevelSequenceActor* OutActor;
	FMovieSceneSequencePlaybackSettings Settings;
	FLevelSequenceCameraSettings CameraSettings;

	ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(InWorld, InSequence, Settings, OutActor);

	FFrameRate Resolution = InSequence->GetMovieScene()->GetTickResolution();
	TRange<FFrameNumber> SpecifiedRange = InRange.ToNative(Resolution);
	Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(SpecifiedRange.GetLowerBoundValue().Value, EUpdatePositionMethod::Play));

	TArray<FSequencerBoundObjects> BoundObjects;

	for (UObject* Object : InObjects)
	{
		TArray<FMovieSceneObjectBindingID> ObjectBindings = Player->GetObjectBindings(Object);
		for (FMovieSceneObjectBindingID ObjectBinding : ObjectBindings)
		{
			FMovieSceneSequenceID SequenceID = ObjectBinding.ResolveSequenceID(MovieSceneSequenceID::Root, *Player);
			FMovieSceneBindingProxy Binding(ObjectBinding.GetGuid(), Player->GetEvaluationState()->FindSequence(SequenceID));
			BoundObjects.Add(FSequencerBoundObjects(Binding, TArray<UObject*>({ Object })));
		}
	}

	Player->Stop();
	InWorld->DestroyActor(OutActor);
	return BoundObjects;
}

void GatherDescendantBindings(const FMovieSceneBindingProxy& Binding, UMovieSceneSequence* Sequence, TArray<FMovieSceneBindingProxy>& AllBindings)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); ++Index)
	{
		FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
		if (Possessable.GetParent() == Binding.BindingID)
		{
			FMovieSceneBindingProxy ChildBinding(Possessable.GetGuid(), Sequence);
			AllBindings.Add(ChildBinding);
			GatherDescendantBindings(ChildBinding, Sequence, AllBindings);
		}
	}
}

bool ExportFBXInternal(const FSequencerExportFBXParams& InParams, UMovieSceneSequencePlayer* Player)
{
	UWorld* World = InParams.World;
	UMovieSceneSequence* Sequence = InParams.Sequence;
	UMovieSceneSequence* RootSequence = InParams.RootSequence;
	TArray<FMovieSceneBindingProxy> BindingProxies = InParams.Bindings;
	TArray<UMovieSceneTrack*> Tracks = InParams.Tracks;
	UFbxExportOption* OverrideOptions = InParams.OverrideOptions;
	FString FBXFileName = InParams.FBXFileName;

	if (!RootSequence)
	{
		RootSequence = Sequence;
	}

	if (!RootSequence)
	{
		return false;
	}

	UnFbx::FFbxExporter* Exporter = UnFbx::FFbxExporter::GetInstance();
	//Show the fbx export dialog options
	Exporter->SetExportOptionsOverride(OverrideOptions);

	UMovieScene* MovieScene = Sequence->GetMovieScene();

	TArray<FMovieSceneBindingProxy> AllBindings;
	for (const FMovieSceneBindingProxy& Binding : BindingProxies)
	{
		AllBindings.Add(Binding);

		GatherDescendantBindings(Binding, Sequence, AllBindings);
	}

	TArray<FGuid> Bindings;
	for (const FMovieSceneBindingProxy& Proxy : AllBindings)
	{
		if (Proxy.Sequence == Sequence)
		{
			Bindings.Add(Proxy.BindingID);
		}
	}

	FMovieSceneEvaluationState* State = Player->GetEvaluationState();
	State->AssignSequence(MovieSceneSequenceID::Root, *RootSequence, *Player);
	FMovieSceneSequenceIDRef Template = State->FindSequenceId(Sequence);
	UnFbx::FFbxExporter::FLevelSequenceNodeNameAdapter NodeNameAdapter(MovieScene, Player, Template);

	FMovieSceneSequenceHierarchy Hierarchy = FMovieSceneSequenceHierarchy();
	UMovieSceneCompiledDataManager::CompileHierarchy(RootSequence, &Hierarchy, EMovieSceneServerClientMask::All);

	const FMovieSceneSubSequenceData* SubSequenceData = Hierarchy.FindSubData(Template);
	FMovieSceneSequenceTransform RootToLocalTransform;

	FFrameTime StartTime = FFrameRate::TransformTime(UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange()).Value, MovieScene->GetTickResolution(), MovieScene->GetDisplayRate());

	if (SubSequenceData)
	{
		RootToLocalTransform = SubSequenceData->RootToSequenceTransform;
		StartTime = RootToLocalTransform.Inverse().TryTransformTime(UE::MovieScene::DiscreteInclusiveLower(SubSequenceData->PlayRange.Value)).Get(StartTime);
	}

	bool bDidExport = false;
	
	{
		FAllSpawnableRestoreState SpawnableRestoreState(Player->GetSharedPlaybackState().ToSharedPtr(), MovieScene);

		if (SpawnableRestoreState.bWasChanged)
		{
			// Evaluate at the beginning of the subscene time to ensure that spawnables are created before export
			Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(StartTime, EUpdatePositionMethod::Play));
		}

		FAnimExportSequenceParameters AESP;
		AESP.Player = Player;
		AESP.RootToLocalTransform = RootToLocalTransform;
		AESP.MovieSceneSequence = Sequence;
		AESP.RootMovieSceneSequence = RootSequence;
		bDidExport = MovieSceneToolHelpers::ExportFBX(World, AESP, Bindings, Tracks, NodeNameAdapter, Template, FBXFileName);
	}

	Player->Stop();
	Exporter->SetExportOptionsOverride(nullptr);
	
	return bDidExport;
}

bool USequencerToolsFunctionLibrary::ExportLevelSequenceFBX(const FSequencerExportFBXParams& InParams)
{
	ALevelSequenceActor* OutActor;
	FMovieSceneSequencePlaybackSettings Settings;
	FLevelSequenceCameraSettings CameraSettings;

	ULevelSequence* RootSequence = InParams.RootSequence;
	if (!RootSequence)
	{
		RootSequence = InParams.Sequence;
	}

	if (!RootSequence)
	{
		FFrame::KismetExecutionMessage(TEXT("Cannot export level sequence. Sequence is invalid."), ELogVerbosity::Error);
		return false;
	}

	ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(InParams.World, RootSequence, Settings, OutActor);

	bool bSuccess = ExportFBXInternal(InParams, Player);
	InParams.World->DestroyActor(OutActor);

	return bSuccess;
}

static USkeletalMeshComponent* GetSkelMeshComponent(IMovieScenePlayer* Player, const FMovieSceneBindingProxy& Binding)
{
	FMovieSceneSequenceIDRef Template = MovieSceneSequenceID::Root;
	for (TWeakObjectPtr<UObject> RuntimeObject : Player->FindBoundObjects(Binding.BindingID, Template))
	{

		if (AActor* Actor = Cast<AActor>(RuntimeObject.Get()))
		{
			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (USkeletalMeshComponent* SkeletalMeshComp = Cast<USkeletalMeshComponent>(Component))
				{
					return SkeletalMeshComp;
				}
			}
		}
		else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(RuntimeObject.Get()))
		{
			if (SkeletalMeshComponent->GetSkeletalMeshAsset())
			{
				return SkeletalMeshComponent;
			}
		}
	}
	return nullptr;
}

bool USequencerToolsFunctionLibrary::ExportAnimSequence(UWorld* World, ULevelSequence* Sequence, UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOptions, const FMovieSceneBindingProxy& Binding, bool bCreateLink)
{
	FCheckCanCompleteDelegate EmptyDelegate;
	return ExportAnimSequenceWaitForDelegate(World, Sequence, AnimSequence, ExportOptions, Binding, bCreateLink, EmptyDelegate);
}

bool USequencerToolsFunctionLibrary::ExportAnimSequenceWaitForDelegate(UWorld* World, ULevelSequence* Sequence, UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOptions, const FMovieSceneBindingProxy& Binding, bool bCreateLink, const FCheckCanCompleteDelegate Delegate)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (Binding.Sequence != Sequence || !AnimSequence)
	{
		return false;
	}

	ALevelSequenceActor* OutActor;
	FMovieSceneSequencePlaybackSettings Settings;
	FLevelSequenceCameraSettings CameraSettings;
	FMovieSceneSequenceIDRef Template = MovieSceneSequenceID::Root;
	FMovieSceneSequenceTransform RootToLocalTransform;
	ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(World, Sequence, Settings, OutActor);

	bool bResult = false;

	FAllSpawnableRestoreState SpawnableRestoreState(Player->GetSharedPlaybackState().ToSharedPtr(), MovieScene);

	if (SpawnableRestoreState.bWasChanged)
	{
		// Evaluate at the beginning of the subscene time to ensure that spawnables are created before export
		FFrameTime StartTime = FFrameRate::TransformTime(UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange()).Value, MovieScene->GetTickResolution(), MovieScene->GetDisplayRate());
		Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(StartTime, EUpdatePositionMethod::Play));

		// Cannot transact changes due to in-flight spawnable transaction
		ExportOptions->bTransactRecording = false;
	}

	auto CompleteExport = [Player, Binding, AnimSequence, RootToLocalTransform, Sequence, ExportOptions, World, OutActor, bCreateLink]() 
	{
		bool bResult = false;

		if (Player && AnimSequence && Sequence && ExportOptions && World)
		{
			USkeletalMeshComponent* SkeletalMeshComp = GetSkelMeshComponent(Player, Binding);
			if (SkeletalMeshComp && SkeletalMeshComp->GetSkeletalMeshAsset() && SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
			{
				AnimSequence->SetSkeleton(SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton());
				FAnimExportSequenceParameters AESQ;
				AESQ.Player = Player;
				AESQ.RootToLocalTransform = RootToLocalTransform;
				AESQ.MovieSceneSequence = Sequence;
				AESQ.RootMovieSceneSequence = Sequence;
				bResult = MovieSceneToolHelpers::ExportToAnimSequence(AnimSequence, ExportOptions, AESQ, SkeletalMeshComp);
			}
			else
			{
				UE_LOGF(LogSequencerTools, Warning, "USequencerToolsFunctionLibrary::ExportAnimSequence: No skel mesh found in Sequencer");
				bResult = false;
			}

			Player->Stop();
			World->DestroyActor(OutActor);
		}
		else
		{
			UE_LOGF(LogSequencerTools, Warning, "USequencerToolsFunctionLibrary::ExportAnimSequence: One or more of the completion lambda's captured parameters are invalid.");
			bResult = false;
		}

		// create the link to the anim sequence
		if (bResult && bCreateLink)
		{
			return LinkAnimSequence(Sequence, AnimSequence, ExportOptions, Binding);
		}

		return bResult;
	};
	
	if (Delegate.IsBound())
	{
		SequencerToolsPrivate::WaitUntil(Delegate, CompleteExport, World);
		return true;
	}
	else
	{
		return CompleteExport();
	}
}

void USequencerToolsFunctionLibrary::ClearLinkedAnimSequences(ULevelSequence* LevelSequence)
{
	if (LevelSequence)
	{
		if (IInterface_AssetUserData* LevelSequenceUserDataInterface = Cast< IInterface_AssetUserData >(LevelSequence))
		{
			ULevelSequenceAnimSequenceLink* LevelAnimLink = LevelSequenceUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
			if (LevelAnimLink)
			{
				for (int32 Index = 0; Index < LevelAnimLink->AnimSequenceLinks.Num(); ++Index)
				{
					FLevelSequenceAnimSequenceLinkItem& LevelAnimLinkItem = LevelAnimLink->AnimSequenceLinks[Index];
					if (UAnimSequence* AnimSequence = LevelAnimLinkItem.ResolveAnimSequence())
					{
						if (IInterface_AssetUserData* AnimAssetUserData = Cast< IInterface_AssetUserData >(AnimSequence))
						{
							UAnimSequenceLevelSequenceLink* AnimLevelLink = AnimAssetUserData->GetAssetUserData< UAnimSequenceLevelSequenceLink >();
							if (AnimLevelLink)
							{
								ULevelSequence* AnimLevelSequence = AnimLevelLink->ResolveLevelSequence();
								if (AnimLevelSequence && AnimLevelSequence == LevelSequence)
								{
									AnimAssetUserData->RemoveUserDataOfClass(UAnimSequenceLevelSequenceLink::StaticClass());
								}
							}
						}
					}
				}
				LevelSequenceUserDataInterface->RemoveUserDataOfClass(ULevelSequenceAnimSequenceLink::StaticClass());			
			}
		}
	}
}

bool USequencerToolsFunctionLibrary::LinkAnimSequence(ULevelSequence*  Sequence,  UAnimSequence* AnimSequence, const UAnimSeqExportOption* ExportOptions,const FMovieSceneBindingProxy& Binding)
{
	if (!Sequence || !AnimSequence || !ExportOptions || Binding.Sequence != Sequence)
	{
		return false;
	}

	if (Sequence && Sequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass())
		&& AnimSequence->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
	{
		Sequence->Modify();
		AnimSequence->Modify();
		if (IInterface_AssetUserData* AnimAssetUserData = Cast< IInterface_AssetUserData >(AnimSequence))
		{
			UAnimSequenceLevelSequenceLink* AnimLevelLink = AnimAssetUserData->GetAssetUserData< UAnimSequenceLevelSequenceLink >();
			if (!AnimLevelLink)
			{
				AnimLevelLink = NewObject<UAnimSequenceLevelSequenceLink>(AnimSequence, NAME_None, RF_Public | RF_Transactional);
				AnimAssetUserData->AddAssetUserData(AnimLevelLink);
			}
			
			AnimLevelLink->SetLevelSequence(Sequence);
			AnimLevelLink->SkelTrackGuid = Binding.BindingID;
		}
		if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(Sequence))
		{
			bool bAddItem = true;
			ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
			if (LevelAnimLink)
			{
				for (FLevelSequenceAnimSequenceLinkItem& LevelAnimLinkItem : LevelAnimLink->AnimSequenceLinks)
				{
					if (LevelAnimLinkItem.IsEqual(Binding.BindingID, ExportOptions->bUseCustomTimeRange,
						ExportOptions->CustomStartFrame, ExportOptions->CustomEndFrame, ExportOptions->CustomDisplayRate,
						ExportOptions->bUseCustomFrameRate, ExportOptions->CustomFrameRate))
					{
						bAddItem = false;
						UAnimSequence* OtherAnimSequence = LevelAnimLinkItem.ResolveAnimSequence();
						
						if (OtherAnimSequence != AnimSequence)
						{
							if (IInterface_AssetUserData* OtherAnimAssetUserData = Cast< IInterface_AssetUserData >(OtherAnimSequence))
							{
								UAnimSequenceLevelSequenceLink* OtherAnimLevelLink = OtherAnimAssetUserData->GetAssetUserData< UAnimSequenceLevelSequenceLink >();
								if (OtherAnimLevelLink)
								{
									OtherAnimSequence->Modify();
									OtherAnimAssetUserData->RemoveUserDataOfClass(UAnimSequenceLevelSequenceLink::StaticClass());
								}
							}
						}
						LevelAnimLinkItem.PathToAnimSequence = FSoftObjectPath(AnimSequence);
						LevelAnimLinkItem.bSetRetargetSourceAsset = ExportOptions->bSetRetargetSourceAsset;
						LevelAnimLinkItem.bExportMorphTargets = ExportOptions->bExportMorphTargets;
						LevelAnimLinkItem.bExportAttributeCurves = ExportOptions->bExportAttributeCurves;
						LevelAnimLinkItem.bExportMaterialCurves = ExportOptions->bExportMaterialCurves;
						LevelAnimLinkItem.bSkipCurvesWithZeroValue = ExportOptions->bSkipCurvesWithZeroValue;
						LevelAnimLinkItem.bExportTransforms = ExportOptions->bExportTransforms;
						LevelAnimLinkItem.Interpolation = ExportOptions->Interpolation;
						LevelAnimLinkItem.CurveInterpolation = ExportOptions->CurveInterpolation;
						LevelAnimLinkItem.bRecordInWorldSpace = ExportOptions->bRecordInWorldSpace;
						LevelAnimLinkItem.bEvaluateAllSkeletalMeshComponents = ExportOptions->bEvaluateAllSkeletalMeshComponents;
						
						LevelAnimLinkItem.IncludeAnimationNames = ExportOptions->IncludeAnimationNames;
						LevelAnimLinkItem.ExcludeAnimationNames = ExportOptions->ExcludeAnimationNames;
						LevelAnimLinkItem.bRemoveExcludedCurves = ExportOptions->bRemoveExcludedCurves;
						LevelAnimLinkItem.WarmUpFrames = ExportOptions->WarmUpFrames;
						LevelAnimLinkItem.DelayBeforeStart = ExportOptions->DelayBeforeStart;
						LevelAnimLinkItem.bUseCustomTimeRange = ExportOptions->bUseCustomTimeRange;
						LevelAnimLinkItem.CustomStartFrame = ExportOptions->CustomStartFrame;
						LevelAnimLinkItem.CustomEndFrame = ExportOptions->CustomEndFrame;
						LevelAnimLinkItem.CustomDisplayRate = ExportOptions->CustomDisplayRate;
						LevelAnimLinkItem.bUseCustomFrameRate = ExportOptions->bUseCustomFrameRate;
						LevelAnimLinkItem.CustomFrameRate = ExportOptions->CustomFrameRate;


						break;
					}
				}
			}
			else
			{
				LevelAnimLink = NewObject<ULevelSequenceAnimSequenceLink>(Sequence, NAME_None, RF_Public | RF_Transactional);
			}
			if (bAddItem == true)
			{
				FLevelSequenceAnimSequenceLinkItem LevelAnimLinkItem;
				LevelAnimLinkItem.SkelTrackGuid = Binding.BindingID;
				LevelAnimLinkItem.PathToAnimSequence = FSoftObjectPath(AnimSequence);
				LevelAnimLinkItem.bSetRetargetSourceAsset = ExportOptions->bSetRetargetSourceAsset;
				LevelAnimLinkItem.bExportMorphTargets = ExportOptions->bExportMorphTargets;
				LevelAnimLinkItem.bExportAttributeCurves = ExportOptions->bExportAttributeCurves;
				LevelAnimLinkItem.Interpolation = ExportOptions->Interpolation;
				LevelAnimLinkItem.CurveInterpolation = ExportOptions->CurveInterpolation;
				LevelAnimLinkItem.bExportMaterialCurves = ExportOptions->bExportMaterialCurves;
				LevelAnimLinkItem.bSkipCurvesWithZeroValue = ExportOptions->bSkipCurvesWithZeroValue;
				LevelAnimLinkItem.bExportTransforms = ExportOptions->bExportTransforms;
				LevelAnimLinkItem.bRecordInWorldSpace = ExportOptions->bRecordInWorldSpace;
				LevelAnimLinkItem.bEvaluateAllSkeletalMeshComponents = ExportOptions->bEvaluateAllSkeletalMeshComponents;

				LevelAnimLinkItem.IncludeAnimationNames = ExportOptions->IncludeAnimationNames;
				LevelAnimLinkItem.ExcludeAnimationNames = ExportOptions->ExcludeAnimationNames;
				LevelAnimLinkItem.bRemoveExcludedCurves = ExportOptions->bRemoveExcludedCurves;
				LevelAnimLinkItem.WarmUpFrames = ExportOptions->WarmUpFrames;
				LevelAnimLinkItem.DelayBeforeStart = ExportOptions->DelayBeforeStart;
				LevelAnimLinkItem.bUseCustomTimeRange = ExportOptions->bUseCustomTimeRange;
				LevelAnimLinkItem.CustomStartFrame = ExportOptions->CustomStartFrame;
				LevelAnimLinkItem.CustomEndFrame = ExportOptions->CustomEndFrame;
				LevelAnimLinkItem.CustomDisplayRate = ExportOptions->CustomDisplayRate;
				LevelAnimLinkItem.bUseCustomFrameRate = ExportOptions->bUseCustomFrameRate;
				LevelAnimLinkItem.CustomFrameRate = ExportOptions->CustomFrameRate;

				LevelAnimLink->AnimSequenceLinks.Add(LevelAnimLinkItem);
				AssetUserDataInterface->AddAssetUserData(LevelAnimLink);
			}
		}
	}
	return true;
}

UAnimSequenceLevelSequenceLink* USequencerToolsFunctionLibrary::GetLevelSequenceLinkFromAnimSequence(UAnimSequence* InAnimSequence)
{
	if (IInterface_AssetUserData* AnimAssetUserData = Cast< IInterface_AssetUserData >(InAnimSequence))
	{
		UAnimSequenceLevelSequenceLink* AnimLevelLink = AnimAssetUserData->GetAssetUserData< UAnimSequenceLevelSequenceLink >();
		return AnimLevelLink;
	}
	return nullptr;
}

ULevelSequenceAnimSequenceLink* USequencerToolsFunctionLibrary::GetAnimSequenceLinkFromLevelSequence(ULevelSequence* InLevelSequence)
{
	if (IInterface_AssetUserData* AssetUserDataInterface = Cast< IInterface_AssetUserData >(InLevelSequence))
	{
		ULevelSequenceAnimSequenceLink* LevelAnimLink = AssetUserDataInterface->GetAssetUserData< ULevelSequenceAnimSequenceLink >();
		return LevelAnimLink;
	}
	return nullptr;
}

TArray<FGuid> AddActors(UWorld* World, UMovieSceneSequence* InSequence, UMovieScene* InMovieScene, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID,const TArray<TWeakObjectPtr<AActor> >& InActors)
{
	TArray<FGuid> PossessableGuids;

	if (InMovieScene->IsReadOnly())
	{
		return PossessableGuids;
	}

	for (TWeakObjectPtr<AActor> WeakActor : InActors)
	{
		if (AActor* Actor = WeakActor.Get())
		{
			FGuid ExistingGuid = Player->FindObjectId(*Actor, TemplateID);
			if (!ExistingGuid.IsValid())
			{
				InMovieScene->Modify();
				const FGuid PossessableGuid = InMovieScene->AddPossessable(Actor->GetActorLabel(), Actor->GetClass());
				PossessableGuids.Add(PossessableGuid);
				InSequence->BindPossessableObject(PossessableGuid, *Actor, World);
				InMovieScene->FindPossessable(PossessableGuid)->FixupPossessedObjectClass(InSequence, World);

				//TODO New to figure way to call void FLevelSequenceEditorToolkit::AddDefaultTracksForActor(AActor& Actor, const FGuid Binding)

				if (Actor->IsA<ACameraActor>())
				{
					MovieSceneToolHelpers::CreateCameraCutSectionForCamera(InMovieScene, PossessableGuid, 0);
				}
			}
		}
	}
	return PossessableGuids;
}


void ImportFBXCamera(UnFbx::FFbxImporter* FbxImporter, UWorld* World, UMovieSceneSequence* Sequence, UMovieScene* InMovieScene, IMovieScenePlayer* Player, FMovieSceneSequenceIDRef TemplateID, TMap<FGuid, FString>& InObjectBindingMap, bool bMatchByNameOnly, bool bCreateCameras)
{
	if (bCreateCameras)
	{
		TArray<FbxCamera*> AllCameras;
		MovieSceneToolHelpers::GetCameras(FbxImporter->Scene->GetRootNode(), AllCameras);

		// Find unmatched cameras
		TArray<FbxCamera*> UnmatchedCameras;
		for (auto Camera : AllCameras)
		{
			FString NodeName = MovieSceneToolHelpers::GetCameraName(Camera);

			bool bMatched = false;
			for (auto InObjectBinding : InObjectBindingMap)
			{
				FString ObjectName = InObjectBinding.Value;
				if (ObjectName == NodeName)
				{
					// Look for a valid bound object, otherwise need to create a new camera and assign this binding to it
					bool bFoundBoundObject = false;
					TArrayView<TWeakObjectPtr<>> BoundObjects = Player->FindBoundObjects(InObjectBinding.Key, TemplateID);
					for (auto BoundObject : BoundObjects)
					{
						if (BoundObject.IsValid())
						{
							bFoundBoundObject = true;
							break;
						}
					}
				}
			}

			if (!bMatched)
			{
				UnmatchedCameras.Add(Camera);
			}
		}

		// If there are new cameras, clear the object binding map so that we're only assigning values to the newly created cameras
		if (UnmatchedCameras.Num() != 0)
		{
			InObjectBindingMap.Reset();
			bMatchByNameOnly = true;
		}

		// Add any unmatched cameras
		for (auto UnmatchedCamera : UnmatchedCameras)
		{
			FString CameraName = MovieSceneToolHelpers::GetCameraName(UnmatchedCamera);

			AActor* NewCamera = nullptr;
			if (UnmatchedCamera->GetApertureMode() == FbxCamera::eFocalLength)
			{
				FActorSpawnParameters SpawnParams;
				NewCamera = World->SpawnActor<ACineCameraActor>(SpawnParams);
				NewCamera->SetActorLabel(*CameraName);
			}
			else
			{
				FActorSpawnParameters SpawnParams;
				NewCamera = World->SpawnActor<ACameraActor>(SpawnParams);
				NewCamera->SetActorLabel(*CameraName);
			}

			// Copy camera properties before adding default tracks so that initial camera properties match and can be restored after sequencer finishes
			MovieSceneToolHelpers::CopyCameraProperties(UnmatchedCamera, NewCamera);

			TArray<TWeakObjectPtr<AActor> > NewCameras;
			NewCameras.Add(NewCamera);
			TArray<FGuid> NewCameraGuids = AddActors(World, Sequence,InMovieScene, Player, TemplateID,NewCameras);

			if (NewCameraGuids.Num())
			{
				InObjectBindingMap.Add(NewCameraGuids[0]);
				InObjectBindingMap[NewCameraGuids[0]] = CameraName;
			}
		}
	}
	//everything created now import it in.
	MovieSceneToolHelpers::ImportFBXCameraToExisting(FbxImporter, Sequence, Player, TemplateID, InObjectBindingMap, bMatchByNameOnly, true);
}

bool ImportFBXInternal(UWorld* World, UMovieSceneSequence* Sequence, const TArray<FMovieSceneBindingProxy>& InBindings, UMovieSceneUserImportFBXSettings* ImportFBXSettings, const FString&  ImportFilename, UMovieSceneSequencePlayer* Player)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene || MovieScene->IsReadOnly())
	{
		return false;
	}

	TArray<FMovieSceneBindingProxy> AllBindings;
	for (const FMovieSceneBindingProxy& Binding : InBindings)
	{
		AllBindings.Add(Binding);

		GatherDescendantBindings(Binding, Sequence, AllBindings);
	}

	TMap<FGuid, FString> ObjectBindingMap;
	for (const FMovieSceneBindingProxy& Binding : AllBindings)
	{
		FString Name = MovieScene->GetObjectDisplayName(Binding.BindingID).ToString();
		ObjectBindingMap.Add(Binding.BindingID, Name);
	}

	FFBXInOutParameters InOutParams;
	if (!MovieSceneToolHelpers::ReadyFBXForImport(ImportFilename, ImportFBXSettings, InOutParams))
	{
		return false;
	}

	const bool bMatchByNameOnly = ImportFBXSettings->bMatchByNameOnly;

	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();
	
	bool bResult = false;
	FScopedTransaction ImportFBXTransaction(NSLOCTEXT("Sequencer", "ImportFBX", "Import FBX"));
	{
		FAllSpawnableRestoreState SpawnableRestoreState(Player->GetSharedPlaybackState().ToSharedPtr(), MovieScene);
 
		if (SpawnableRestoreState.bWasChanged)
		{
			// Evaluate at the beginning of the subscene time to ensure that spawnables are created before export
			FFrameTime StartTime = FFrameRate::TransformTime(UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange()).Value, MovieScene->GetTickResolution(), MovieScene->GetDisplayRate());
			Player->SetPlaybackPosition(FMovieSceneSequencePlaybackParams(StartTime, EUpdatePositionMethod::Play));
		}

		ImportFBXCamera(FbxImporter, World, Sequence, MovieScene, Player, MovieSceneSequenceID::Root, ObjectBindingMap, bMatchByNameOnly, ImportFBXSettings->bCreateCameras);

		bResult = MovieSceneToolHelpers::ImportFBXIfReady(World, Sequence, Player, MovieSceneSequenceID::Root, ObjectBindingMap, ImportFBXSettings, InOutParams);
	}
	
	Player->Stop();
	return bResult;
}

bool USequencerToolsFunctionLibrary::ImportLevelSequenceFBX(UWorld* World, ULevelSequence* Sequence, const TArray<FMovieSceneBindingProxy>& InBindings, UMovieSceneUserImportFBXSettings* ImportFBXSettings, const FString&  ImportFilename, ALevelSequenceActor* ActorContext)
{
	ALevelSequenceActor* OutActor = nullptr;
	FMovieSceneSequencePlaybackSettings Settings;
	FLevelSequenceCameraSettings CameraSettings;
	ULevelSequencePlayer* Player = ActorContext ? ActorContext->GetSequencePlayer() : ULevelSequencePlayer::CreateLevelSequencePlayer(World, Sequence, Settings, OutActor);

	bool bSuccess = ImportFBXInternal(World, Sequence, InBindings, ImportFBXSettings, ImportFilename, Player);
	if(OutActor)
	{
		World->DestroyActor(OutActor);
	}

	return bSuccess;
}

bool USequencerToolsFunctionLibrary::ImportFBXToControlRig(UWorld* World, ULevelSequence* Sequence, const FString& ControlRigTrackName, const TArray<FString>& ControlRigNames,
	UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings,
	const FString& ImportFilename)
{
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene || MovieScene->IsReadOnly())
	{
		return false;
	}

	bool bValid = false;

	const TArray<FMovieSceneBinding>& Bindings =  ((const UMovieScene*)MovieScene)->GetBindings();



	for (const FMovieSceneBinding& Binding : Bindings)
	{
		if (MovieScene->GetObjectDisplayName(Binding.GetObjectGuid()).ToString() == ControlRigTrackName)
		{
			
			ALevelSequenceActor* OutActor;
			FMovieSceneSequencePlaybackSettings Settings;
			FLevelSequenceCameraSettings CameraSettings;
			ULevelSequencePlayer* Player = ULevelSequencePlayer::CreateLevelSequencePlayer(World, Sequence, Settings, OutActor);

			const TArray<UMovieSceneTrack*>& Tracks = Binding.GetTracks();
			TArray<FName> SelectedControls;
			for (UMovieSceneTrack* Track : Tracks)
			{
				INodeAndChannelMappings* ChannelMapping = Cast<INodeAndChannelMappings>(Track); 
				if (ChannelMapping)
				{
					TArray<FRigControlFBXNodeAndChannels>* NodeAndChannels = ChannelMapping->GetNodeAndChannelMappings(nullptr);
					//use passed in controls for selected, actually selected controls should almost be empty anyway since we just loaded/set everything up.
					for (const FString& StringName : ControlRigNames)
					{
						FName Name(*StringName);
						SelectedControls.Add(Name);
					}

					bValid = MovieSceneToolHelpers::ImportFBXIntoControlRigChannels(MovieScene,ImportFilename, ImportFBXControlRigSettings,
					NodeAndChannels, SelectedControls, MovieScene->GetTickResolution());

					if (NodeAndChannels)
					{
						delete NodeAndChannels;
					}
				}
			}
			return bValid;
		}
	}
	
	return false;
	

}

bool USequencerToolsFunctionLibrary::ExportFBXFromControlRig(ULevelSequence* Sequence,
	const FString& ActorWithControlRigTrack,
	const UMovieSceneUserExportFBXControlRigSettings* ExportFBXControlRigSettings)
{
	bool bValid = false;

	if (!Sequence || !ExportFBXControlRigSettings)
	{
		return false;
	}
	
	UMovieScene* MovieScene = Sequence->GetMovieScene();
	if (!MovieScene || MovieScene->IsReadOnly())
	{
		return false;
	}

	const TArray<FMovieSceneBinding>& Bindings = ((const UMovieScene*)MovieScene)->GetBindings();
	for (const FMovieSceneBinding& Binding : Bindings)
	{
		if (MovieScene->GetObjectDisplayName(Binding.GetObjectGuid()).ToString() != ActorWithControlRigTrack)
		{
			continue;
		}

		for (UMovieSceneTrack* Track : Binding.GetTracks())
		{
			INodeAndChannelMappings* ChannelMapping = Cast<INodeAndChannelMappings>(Track);
			const UMovieSceneSection* Section = Track->GetSectionToKey();
			if (!ChannelMapping || !Section)
			{
				continue;
			}

			TArray<FName> SelectedControls;
			ChannelMapping->GetSelectedNodes(SelectedControls);

			const FMovieSceneSequenceTransform RootToLocalTransform;
			return MovieSceneToolHelpers::ExportFBXFromControlRigChannels(Section, ExportFBXControlRigSettings, SelectedControls, RootToLocalTransform);
		}
	}

	return false;
}


FMovieSceneEvent USequencerToolsFunctionLibrary::CreateEvent(UMovieSceneSequence* InSequence, UMovieSceneEventSectionBase* InSection, const FSequencerQuickBindingResult& InEndpoint, const TArray<FString>& InPayload)
{
	FMovieSceneEvent Event;

	if (InEndpoint.EventEndpoint == nullptr)
	{
		FFrame::KismetExecutionMessage(TEXT("Invalid endpoint, event will not be initialized"), ELogVerbosity::Warning);
		return Event;
	}

	UMovieScene* MovieScene = InSequence->GetMovieScene();
	FGuid ObjectBindingID;
	MovieScene->FindTrackBinding(*InSection->GetTypedOuter<UMovieSceneTrack>(), ObjectBindingID);
	UClass* BoundObjectPinClass = nullptr;
	if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBindingID))
	{
		BoundObjectPinClass = const_cast<UClass*>(Possessable->GetPossessedObjectClass());
	}
	else if (FMovieSceneSpawnable* Spawnable = MovieScene->FindSpawnable(ObjectBindingID))
	{
		BoundObjectPinClass = Spawnable->GetObjectTemplate()->GetClass();
	}

	InSection->Modify();
	FMovieSceneEventUtils::BindEventSectionToBlueprint(InSection, InEndpoint.EventEndpoint->GetBlueprint());

	UEdGraphPin* BoundObjectPin = FMovieSceneDirectorBlueprintUtils::FindCallTargetPin(InEndpoint.EventEndpoint, BoundObjectPinClass);
	FMovieSceneEventUtils::SetEndpoint(&Event, InSection, InEndpoint.EventEndpoint, BoundObjectPin);

	if (InEndpoint.PayloadNames.Num() != InPayload.Num())
	{
		const FString Message = FString::Printf(TEXT("Wrong number of payload values, expecting %i got %i"), InEndpoint.PayloadNames.Num(), InPayload.Num());
		FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
		return Event;
	}

	for (int32 Index = 0; Index < InEndpoint.PayloadNames.Num(); Index++)
	{
		const FName PayloadName = FName(InEndpoint.PayloadNames[Index]);
		if (!Event.PayloadVariables.Contains(PayloadName))
		{
			Event.PayloadVariables.Add(PayloadName);
			Event.PayloadVariables[PayloadName].Value = InPayload[Index];
		}
	}

	return Event;
}


bool USequencerToolsFunctionLibrary::IsEventEndpointValid(const FSequencerQuickBindingResult& InEndpoint)
{
	return InEndpoint.EventEndpoint != nullptr;
}


FSequencerQuickBindingResult USequencerToolsFunctionLibrary::CreateQuickBinding(UMovieSceneSequence* InSequence, UObject* InObject, const FString& InFunctionName, bool bCallInEditor)
{
	FSequencerQuickBindingResult Result;

	FMovieSceneSequenceEditor* SequenceEditor = FMovieSceneSequenceEditor::Find(InSequence);
	if (!SequenceEditor)
	{
		return Result;
	}

	UBlueprint* Blueprint = SequenceEditor->GetOrCreateDirectorBlueprint(InSequence);
	if (!Blueprint)
	{
		return Result;
	}

	UMovieScene* MovieScene = InSequence->GetMovieScene();
	
	FMovieSceneDirectorBlueprintEndpointDefinition EndpointDefinition;
	EndpointDefinition.EndpointType = EMovieSceneDirectorBlueprintEndpointType::Event;
	EndpointDefinition.EndpointName = InFunctionName;
	EndpointDefinition.PossibleCallTargetClass = InObject->GetClass();
	EndpointDefinition.AddExtraOutputPin(InObject->GetName(), UEdGraphSchema_K2::PC_Object, InObject->GetClass());
	UFunction* Function = InObject->GetClass()->FindFunctionByName(FName(InFunctionName));
	if (Function == nullptr)
	{
		const FString Message = FString::Printf(TEXT("Cannot find function %s in class %s"), *(InFunctionName), *(InObject->GetClass()->GetName()));
		FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
		return Result;
	}

	UBlueprintFunctionNodeSpawner* BlueprintFunctionNodeSpawner = UBlueprintFunctionNodeSpawner::Create(Function);
	FBlueprintActionMenuItem Action(BlueprintFunctionNodeSpawner);

	UK2Node_CustomEvent* NewEventEndpoint = FMovieSceneDirectorBlueprintUtils::CreateEventEndpoint(Blueprint, EndpointDefinition);
	NewEventEndpoint->bCallInEditor = bCallInEditor;
	Result.EventEndpoint = NewEventEndpoint;

	UEdGraphPin* ThenPin = NewEventEndpoint->FindPin(UEdGraphSchema_K2::PN_Then, EGPD_Output);
	UEdGraphPin* BoundObjectPin = FMovieSceneDirectorBlueprintUtils::FindCallTargetPin(NewEventEndpoint, EndpointDefinition.PossibleCallTargetClass);

	FVector2f NodePosition(NewEventEndpoint->NodePosX + 400.f, NewEventEndpoint->NodePosY);
	UEdGraphNode* NewNode = Action.PerformAction(NewEventEndpoint->GetGraph(), BoundObjectPin ? BoundObjectPin : ThenPin, NodePosition);

	if (NewNode == nullptr)
	{
		const FString Message = FString::Printf(TEXT("Failed creating blueprint event node for function %s"), *InFunctionName);
		FFrame::KismetExecutionMessage(*Message, ELogVerbosity::Warning);
		return Result;
	}

	// Link execution pins
	UEdGraphPin* ExecPin = NewNode->FindPin(UEdGraphSchema_K2::PN_Execute, EGPD_Input);
	if (ensure(ThenPin && ExecPin))
	{
		ThenPin->MakeLinkTo(ExecPin);
	}

	// Link payload parameters' pins
	UK2Node_EditablePinBase* EditableNode = Cast<UK2Node_EditablePinBase>(NewEventEndpoint);
	if (EditableNode)
	{
		for (UEdGraphPin* PayloadPin : NewNode->Pins)
		{
			if (PayloadPin != BoundObjectPin && PayloadPin->Direction == EGPD_Input && PayloadPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Exec && PayloadPin->LinkedTo.Num() == 0)
			{
				Result.PayloadNames.Add(PayloadPin->PinName.ToString());

				UEdGraphPin* NewPin = EditableNode->CreateUserDefinedPin(PayloadPin->PinName, PayloadPin->PinType, EGPD_Output);
				if (NewNode != NewEventEndpoint && NewPin)
				{
					NewPin->MakeLinkTo(PayloadPin);
				}
			}
		}
	}

	return Result;
}


#undef LOCTEXT_NAMESPACE // "SequencerTools"
