// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "SequencerCurveEditorObject.generated.h"

#define UE_API SEQUENCERSCRIPTINGEDITOR_API


class ISequencer;
class ULevelSequence;
class FCurveEditor;
struct FMovieSceneChannel;
class UMovieSceneSection;
struct FCurveModelID;
class UCurveEditorFilterBase;

USTRUCT(BlueprintType)
struct FSequencerChannelProxy
{
	GENERATED_BODY()

	FSequencerChannelProxy()
		: Section(nullptr)
	{}

	FSequencerChannelProxy(const FName& InChannelName, UMovieSceneSection* InSection)
		: ChannelName(InChannelName)
		, Section(InSection)
	{}

	UPROPERTY(BlueprintReadWrite, Category = Channel)
	FName ChannelName;

	UPROPERTY(BlueprintReadWrite, Category = Channel)
	TObjectPtr<UMovieSceneSection> Section;
};


/*
* Class to hold sequencer curve editor functions
*/
UCLASS(MinimalAPI)
class USequencerCurveEditorObject : public UObject
{
public:

	GENERATED_BODY()

public:
	/** Open curve editor*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API void OpenCurveEditor();

	UFUNCTION(BlueprintPure, Category = "Sequencer Curve Editor")
	/** Is curve editor open*/
	UE_API bool IsCurveEditorOpen();

	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	/** Close curve editor*/
	UE_API void CloseCurveEditor();

	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API void ApplyFilter(UCurveEditorFilterBase* Filter);

public:

	/** Gets the channel with selected keys */
	UFUNCTION(BlueprintPure, Category = "Sequencer Curve Editor")
	UE_API TArray<FSequencerChannelProxy> GetChannelsWithSelectedKeys();

	/** Gets the selected keys with this channel */
	UFUNCTION(BlueprintPure, Category = "Sequencer Curve Editor")
	UE_API TArray<int32> GetSelectedKeys(const FSequencerChannelProxy& ChannelProxy);

	/** Show curve  */
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API void ShowCurve(const FSequencerChannelProxy& Channel, bool bShowCurve);

	/** Is the curve displayed*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API bool IsCurveShown(const FSequencerChannelProxy& Channel);

	/** Select keys */
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API void SelectKeys(const FSequencerChannelProxy& Channel, const TArray<int32>& Indices);

	/** Empties the current selection. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API void EmptySelection();

public:

	/** Get if a custom color for specified channel idendified by it's class and identifier exists */
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API bool HasCustomColorForChannel(UClass* Class, const FString& Identifier);

	/** Get custom color for specified channel idendified by it's class and identifier,if none exists will return white*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API FLinearColor GetCustomColorForChannel(UClass* Class, const FString& Identifier);

	/** Set Custom Color for specified channel idendified by it's class and identifier. This will be stored in editor user preferences.*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API void SetCustomColorForChannel(UClass* Class, const FString& Identifier, const FLinearColor& NewColor);

	/** Set Custom Color for specified channels idendified by it's class and identifiers. This will be stored in editor user preferences.*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API void SetCustomColorForChannels(UClass* Class, const TArray<FString>& Identifiers, const TArray<FLinearColor>& NewColors);

	/** Set Random Colors for specified channels idendified by it's class and identifiers. This will be stored in editor user preferences.*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API void SetRandomColorForChannels(UClass* Class, const TArray<FString>& Identifiers);

	/** Delete for specified channel idendified by it's class and identifier.*/
	UFUNCTION(BlueprintCallable, Category = "Sequencer Curve Editor")
	UE_API void DeleteColorForChannels(UClass* Class, FString& Identifier);

public:

	/**
	 * Function to assign a sequencer singleton.
	 * NOTE: Only to be called by ULevelSequenceBlueprintLibrary::SetSequencer.
	 */
	UE_API void SetSequencer(TSharedPtr<ISequencer>& InSequencer);

public:

	/**
	 * Utility function to get curve, if it exists,  from a section and a name
	 */
	UE_API TOptional<FCurveModelID> GetCurve(UMovieSceneSection* InSection, const FName& InName);

	/**
	Utility function to get curve editor
	*/
	UE_API TSharedPtr<FCurveEditor> GetCurveEditor();

private:
	//internal sequencer
	TWeakPtr<ISequencer> CurrentSequencer;
};

#undef UE_API
