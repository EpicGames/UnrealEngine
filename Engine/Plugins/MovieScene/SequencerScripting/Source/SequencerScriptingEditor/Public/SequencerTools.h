// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MovieSceneCaptureDialogModule.h"
#include "MovieSceneBindingProxy.h"
#include "MovieSceneTrack.h"
#include "SequencerTools.generated.h"

#define UE_API SEQUENCERSCRIPTINGEDITOR_API

class IModuleInterface;
class ULevelSequence;
class UMovieSceneUserImportFBXSettings;
struct FMovieSceneEvent;
struct FSequencerScriptingRange;

class UFbxExportOption;
class UAnimSequenceExportOption;
class UAnimSequence;
class UPoseAsset;
class UMovieSceneSequencePlayer;

class UMovieSceneEventSectionBase;
class UK2Node_CustomEvent;

class UAnimSeqExportOption;
class UMovieSceneUserImportFBXControlRigSettings;

class ULevelSequenceAnimSequenceLink;
class UAnimSequenceLevelSequenceLink;

DECLARE_DYNAMIC_DELEGATE_OneParam(FOnRenderMovieStopped, bool, bSuccess);
DECLARE_DYNAMIC_DELEGATE_RetVal(bool, FCheckCanCompleteDelegate);


USTRUCT(BlueprintType)
struct FSequencerBoundObjects
{
	GENERATED_BODY()

	FSequencerBoundObjects()
	{}

	FSequencerBoundObjects(FMovieSceneBindingProxy InBindingProxy, const TArray<UObject*>& InBoundObjects)
		: BindingProxy(InBindingProxy)
		, BoundObjects(InBoundObjects)
	{}

	UPROPERTY(BlueprintReadWrite, Category=Binding)
	FMovieSceneBindingProxy BindingProxy;

	UPROPERTY(BlueprintReadWrite, Category=Binding)
	TArray<TObjectPtr<UObject>> BoundObjects;
};

/** Wrapper around result of quick binding for event track in sequencer. */
USTRUCT(BlueprintType)
struct FSequencerQuickBindingResult
{
	GENERATED_BODY()

	FSequencerQuickBindingResult() : EventEndpoint(nullptr) {}

	/** Actual endpoint wrapped by this structure.  */
	UPROPERTY()
	TObjectPtr<UK2Node_CustomEvent> EventEndpoint;

	/** Names of the payload variables of the event. */
	UPROPERTY(BlueprintReadOnly, Category = Data)
	TArray<FString> PayloadNames;

};

USTRUCT(BlueprintType)
struct FSequencerExportFBXParams
{
	GENERATED_BODY()

	FSequencerExportFBXParams() {}
	FSequencerExportFBXParams(UWorld* InWorld, ULevelSequence* InSequence, ULevelSequence* InRootSequence, const TArray<FMovieSceneBindingProxy>& InBindings, const TArray<UMovieSceneTrack*>& InTracks, UFbxExportOption* InOverrideOptions, const FString& InFBXFileName)
		: World(InWorld)
		, Sequence(InSequence)
		, RootSequence(InRootSequence)
		, Bindings(InBindings)
		, Tracks(InTracks)
		, OverrideOptions(InOverrideOptions)
		, FBXFileName(InFBXFileName) {}
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<UWorld> World;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<ULevelSequence> Sequence;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<ULevelSequence> RootSequence;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TArray<FMovieSceneBindingProxy> Bindings;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TArray<TObjectPtr<UMovieSceneTrack>> Tracks;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	TObjectPtr<UFbxExportOption> OverrideOptions;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Movie Scene")
	FString FBXFileName;
};

/** 
 * This is a set of helper functions to access various parts of the Sequencer API via Python. Because Sequencer itself is not suitable for exposing, most functionality
 * gets wrapped by UObjects that have an easier API to work with. This UObject provides access to these wrapper UObjects where needed. 
 */
UCLASS(MinimalAPI, Transient, meta=(ScriptName="SequencerTools"))
class USequencerToolsFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	/**
	* Attempts to render a sequence to movie based on the specified settings. This will automatically detect
	* if we're rendering via a PIE instance or a new process based on the passed in settings. Will return false
	* if the state is not valid (ie: null or missing required parameters, capture in progress, etc.), true otherwise.
	*/
	UE_DEPRECATED(5.3, "SequencerTools::RenderMovie has been deprecated, please use Movie Render Queue")
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Movie Rendering", meta=(DeprecatedFunction))
	static UE_API bool RenderMovie(class UMovieSceneCapture* InCaptureSettings, FOnRenderMovieStopped OnFinishedCallback);

	/** 
	* Returns if Render to Movie is currently in progress.
	*/
	UE_DEPRECATED(5.3, "SequencerTools::IsRenderingMovie has been deprecated, please use Movie Render Queue")
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Movie Rendering", meta=(DeprecatedFunction))
	static bool IsRenderingMovie()
	{
		IMovieSceneCaptureDialogModule& MovieSceneCaptureModule = FModuleManager::Get().LoadModuleChecked<IMovieSceneCaptureDialogModule>("MovieSceneCaptureDialog");
		return MovieSceneCaptureModule.GetCurrentCapture().IsValid();
	}

	/**
	* Attempts to cancel an in-progress Render to Movie. Does nothing if there is no render in progress.
	*/
	UE_DEPRECATED(5.3, "SequencerTools::CancelMovieRender has been deprecated, please use Movie Render Queue")
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Movie Rendering", meta=(DeprecatedFunction))
	static UE_API void CancelMovieRender();

public:

	/*
	 * Retrieve all objects currently bound to the specified binding identifiers. The sequence will be evaluated in lower bound of the specified range, 
	 * which allows for retrieving spawnables in that period of time.
	 */
	UE_DEPRECATED(5.3, "SequencerTools::GetBoundObjects has been deprecated because it doesn't work with Spawnables, please use ULevelSequenceEditorBlueprintLibrary::GetBoundObjects which operates on the currently opened sequence in Editor")
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools", meta=(DeprecatedFunction))
	static UE_API TArray<FSequencerBoundObjects> GetBoundObjects(UWorld* InWorld, ULevelSequence* InSequence, const TArray<FMovieSceneBindingProxy>& InBindings, const FSequencerScriptingRange& InRange);

	/*
	 * Get the object bindings for the requested object. The sequence will be evaluated in lower bound of the specified range, 
	 * which allows for retrieving spawnables in that period of time.
	 */
	UE_DEPRECATED(5.3, "SequencerTools::GetObjectBindings has been deprecated because it doesn't work with Spawnables, please use ULevelSequenceEditorBlueprintLibrary::GetObjectBindings which operates on the currently opened sequence in Editor")
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools", meta=(DeprecatedFunction))
	static UE_API TArray<FSequencerBoundObjects> GetObjectBindings(UWorld* InWorld, ULevelSequence* InSequence, const TArray<UObject*>& InObject, const FSequencerScriptingRange& InRange);

public:

	/*
	 * Export Passed in Bindings and Tracks to FBX
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | FBX")
	static UE_API bool ExportLevelSequenceFBX(const FSequencerExportFBXParams& InParams);
	
	/*
	 * Export Passed in Binding as an Anim Sequence. Note use this when you want to export without an open Level Sequence in Sequencer, and
	 * know that all spawnables that may be dynamically created and destroyed will be valid. If the spawnable won't be fully created and you have
	 * an open Sequencer use UControlRigSequencerEditorLibrary::ExportAnimSequenceFromSequencer instead.
	 *
	 * @InWorld World to export
	 * @InSequence Sequence to export
	 * @AnimSequence The AnimSequence to save into.
	 * @ExportOption The export options for the sequence.
	 * @InBinding Binding to export that has a skelmesh component on it
	 * @InAnimSequenceFilename File to create
	 * @bCreateLink If true will create a link between the animation sequence and the level sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API bool ExportAnimSequence(UWorld* World, ULevelSequence*  Sequence, UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOptions, const FMovieSceneBindingProxy& Binding, bool bCreateLink);
	
	/*
	* Version of ExportAnimSequence, which will spawn in all actors in the sequence and then wait for the provided Delegate to return true before exporting.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API bool ExportAnimSequenceWaitForDelegate(UWorld* World, ULevelSequence* Sequence, UAnimSequence* AnimSequence, UAnimSeqExportOption* ExportOptions, const FMovieSceneBindingProxy& Binding, bool bCreateLink, const FCheckCanCompleteDelegate Delegate);

	/*
	 * Links a LevelSequence's SkeletalMesh binding to an existing anim sequence.
	 *
	 * @InSequence Sequence to link from
	 * @AnimSequence The AnimSequence to link to.
	 * @ExportOption The export options that should be used when baking the LevelSequence.
	 * @InBinding Binding that has a skelmesh component on it
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API bool LinkAnimSequence(ULevelSequence* Sequence, UAnimSequence* AnimSequence, const UAnimSeqExportOption* ExportOptions, const FMovieSceneBindingProxy& Binding);

	/*
	 * Clear all linked anim sequences between this level sequence and any linked anim sequences
	 *
	 * @InLevelSequence LevelSequence to clean links for
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API void ClearLinkedAnimSequences(ULevelSequence* InLevelSequence);

	/*
	 * Get the link to the level sequence if it exists on this anim sequence
	 *
	 * @InAnimSequence AnimSequence to get links from
	 * @return Returns the link object if it exists, nullptr if it doesn't
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API UAnimSequenceLevelSequenceLink* GetLevelSequenceLinkFromAnimSequence(UAnimSequence* InAnimSequence);

	/*
	 * Get the links to the anim sequences if they exist on this level sequence
	 *
	 * @InLevelSequence LevelSequence to get links from
	 * @return Returns the link object if it exists, nullptr if it doesn't
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API ULevelSequenceAnimSequenceLink* GetAnimSequenceLinkFromLevelSequence(ULevelSequence* InLevelSequence);

	/*
	 * Import FBX onto Passed in Bindings
	 *
	 * @InWorld World to import to
	 * @InSequence InSequence to import
	 * @InBindings InBindings to import
	 * @InImportFBXSettings Settings to control import.
	 * @InImportFileName Path to fbx file to import from
	 * @InPlayer Player to bind to
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | FBX")
	static UE_API bool ImportLevelSequenceFBX(UWorld* InWorld, ULevelSequence* InSequence, const TArray<FMovieSceneBindingProxy>& InBindings, UMovieSceneUserImportFBXSettings* InImportFBXSettings, const FString& InImportFilename, ALevelSequenceActor* ActorContext = nullptr);

public:
	/**
	 * Create an event from a previously created blueprint endpoint and a payload. The resulting event should be added only
	 * to a channel of the section that was given as a parameter.
	 * @param InSequence Main level sequence that holds the event track and to which the resulting event should be added.
	 * @param InSection Section of the event track of the main sequence.
	 * @param InEndpoint Previously created endpoint.
	 * @param InPayload Values passed as payload to event, count must match the numbers of payload variable names in @InEndpoint.
	 * @return The created movie event.
	 * @see CreateQuickBinding
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API FMovieSceneEvent CreateEvent(UMovieSceneSequence* InSequence, UMovieSceneEventSectionBase* InSection, const FSequencerQuickBindingResult& InEndpoint, const TArray<FString>& InPayload);

	/**
	 * Check if an endpoint is valid and can be used to create movie scene event.
	 * @param InEndpoint Endpoint to check.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API bool IsEventEndpointValid(const FSequencerQuickBindingResult& InEndpoint);

	/**
	 * Create a quick binding to an actor's member method to be used in an event sequence.
	 * @param InActor Actor that will be bound
	 * @param InFunctionName Name of the method, as it is displayed in the Blueprint Editor. eg. "Set Actor Scale 3D"
	 * @param bCallInEditor Should the event be callable in editor.
	 * @return The created binding.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | Animation")
	static UE_API FSequencerQuickBindingResult CreateQuickBinding(UMovieSceneSequence* InSequence, UObject* InObject, const FString& InFunctionName, bool bCallInEditor);

	/*
	 * Import FBX onto a control rig with the specified track name
	 *
	 * @InWorld World to import to
	 * @InSequence InSequence to import
	 * @ActorWithControlRigTrack ActorWithControlRigTrack The name of the actor with the control rig track we are importing onto
	 * @SelectedControlRigNames  List of selected control rig names. Will use them if  ImportFBXControlRigSettings->bImportOntoSelectedControls is true
	 * @ImportFBXControlRigSettings Settings to control import.
	 * @InImportFileName Path to fbx file to create
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | FBX")
	static UE_API bool ImportFBXToControlRig(UWorld* World, ULevelSequence* InSequence, const FString& ActorWithControlRigTrack, const TArray<FString>& SelectedControlRigNames,
		UMovieSceneUserImportFBXControlRigSettings* ImportFBXControlRigSettings,
		const FString& ImportFilename);

	/** Exports an FBX from the section to key of the control rig track of the actor with the given name. */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Sequencer Tools | FBX")
	static UE_API bool ExportFBXFromControlRig(ULevelSequence* Sequence, const FString& ActorWithControlRigTrack, const UMovieSceneUserExportFBXControlRigSettings* ExportFBXControlRigSettings);
};

#undef UE_API
