// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SequencerScriptingEditor : ModuleRules
{
	public SequencerScriptingEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
            }
		);

		PrivateIncludePaths.AddRange(
			new string[] {
                "../../../../Source/Editor/UnrealEd/Private", // TODO: Fix this, for now it's needed for the fbx exporter
            }
        );

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"TimeManagement",
				"MovieScene",
				"MovieSceneCaptureDialog",
				"MovieSceneTools",
                "MovieSceneTracks",
				"CinematicCamera",
				"SequencerScripting"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Kismet",
				"PythonScriptPlugin",
				"Slate",
				"SlateCore",
                "MovieSceneCapture",
                "LevelSequence",
				"EditorFramework",
                "UnrealEd",
                "Sequencer",
                "BlueprintGraph",
				"CurveEditor",
				"SequencerCore"
            }
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				
			}
		);
        AddEngineThirdPartyPrivateStaticDependencies(Target, "FBX");

    }
}
